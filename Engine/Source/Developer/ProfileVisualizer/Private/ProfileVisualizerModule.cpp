// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileVisualizerModule.h"
#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/SlateWindowHelper.h"
#include "Framework/Docking/TabManager.h"
#include "Tickable.h"
#include "SProfileVisualizer.h"
#include "Widgets/Docking/SDockTab.h"
#include "ProfileVisualizerStyle.h"
#if WITH_EDITOR
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Styling/AppStyle.h"
#endif //#if WITH_EDITOR

/**
 * Creates Visualizer using Visualizer profile data format
 *
 * @param ProfileData Visualizer data
 * @return Visualizer window
 */

FName ProfileVisualizerTabId("VisualizerSpawnPoint");
TSharedRef<SDockTab> MakeProfileVisualizerWindow( TSharedPtr< FVisualizerEvent > ProfileData, const FText& WindowTitle, const FText& ProfilerType, const FText& HeaderMessageText = FText::GetEmpty(), const FLinearColor& HeaderMessageTextColor = FLinearColor::White, const bool InsertTab = true )
{
	TSharedRef<SDockTab> ProfileVisualizer = SNew(SDockTab)
												.Label(WindowTitle)
												.TabRole(ETabRole::NomadTab)
												[
													SNew(SProfileVisualizer)
													.ProfileData(ProfileData)
													.ProfilerType(ProfilerType)
													.HeaderMessageText(HeaderMessageText)
													.HeaderMessageTextColor(HeaderMessageTextColor)
												];

	if (InsertTab)
	{
		FGlobalTabmanager::Get()->InsertNewDocumentTab
		(
			ProfileVisualizerTabId, FTabManager::ESearchPreference::RequireClosedTab,
			ProfileVisualizer
		);
	}

	return ProfileVisualizer;
}

/** Helper class that counts down when to unpause and stop movie. */
class FDelayedVisualizerSpawner : public FTickableGameObject
{
public:

	struct FPendingWindow
	{
		FText Title;
		FText Type;
		TSharedPtr< FVisualizerEvent > ProfileData;

		FPendingWindow( TSharedPtr<FVisualizerEvent> InData, const FText& InTitle, const FText& InType )
			: Title( InTitle )
			, Type( InType )
			, ProfileData( InData )
		{}
	};

	FDelayedVisualizerSpawner( )
	{
	}

	virtual ~FDelayedVisualizerSpawner()
	{
	}

	void AddPendingData( TSharedPtr<FVisualizerEvent> InProfileData, const FText& InTitle, const FText& InType )
	{
		UE::TScopeLock ScopeLock(DataLock);

		VisualizerDataToSpawn.Add( MakeShareable( new FPendingWindow( InProfileData, InTitle, InType ) )  );
	}

	// FTickableGameObject interface

	void Tick(float DeltaTime) override
	{
		UE::TScopeLock ScopeLock(DataLock);

		for ( int32 Index = 0; Index < VisualizerDataToSpawn.Num(); Index++ )
		{
			TSharedPtr<FPendingWindow> Window =  VisualizerDataToSpawn[ Index ];
			MakeProfileVisualizerWindow( Window->ProfileData, Window->Title, Window->Type ); 
		}

		VisualizerDataToSpawn.Empty();
	}

	/** We should call Tick on this object */
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	/** Need this to be ticked when paused (that is the point!) */
	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FDelayedVisualizerSpawner, STATGROUP_Tickables);
	}

private:

	FTransactionallySafeCriticalSection DataLock;

	/** List of profile data sets to spawn visualizers for */
	TArray< TSharedPtr<FPendingWindow> > VisualizerDataToSpawn;
};

static TSharedPtr< FDelayedVisualizerSpawner > GDelayedVisualizerSpawner;

void InitProfileVisualizer()
{
	FProfileVisualizerStyle::Initialize();
	if( GDelayedVisualizerSpawner.IsValid() == false )
	{
		GDelayedVisualizerSpawner = MakeShareable( new FDelayedVisualizerSpawner() );
	}
}

void ShutdownProfileVisualizer()
{
	FProfileVisualizerStyle::Shutdown();
	GDelayedVisualizerSpawner.Reset();
}

static bool GHasRegisteredVisualizerLayout = false;

void DisplayProfileVisualizer(TSharedPtr< FVisualizerEvent > InProfileData, const TCHAR* InProfilerType, const FText& HeaderMessageText = FText::GetEmpty(), const FLinearColor& HeaderMessageTextColor = FLinearColor::White)
{
	check( IsInGameThread() );

	FFormatNamedArguments Args;
	Args.Add( TEXT("ProfilerType"), FText::FromString( InProfilerType ) );

	const FText WindowTitle = FText::Format( NSLOCTEXT("ProfileVisualizer", "WindowTitle", "{ProfilerType} Visualizer"), Args );
	const FText ProfilerType = FText::Format( NSLOCTEXT("ProfileVisualizer", "ProfilerType", "{ProfilerType} Profile"), Args );

	MakeProfileVisualizerWindow( InProfileData, WindowTitle, ProfilerType, HeaderMessageText, HeaderMessageTextColor );
	
}

/**
 * Module for profile visualizer.
 */
class FProfileVisualizerModule : public IProfileVisualizerModule
{
public:
	virtual void StartupModule() override
	{
		::InitProfileVisualizer();

#if WITH_EDITOR
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ProfileVisualizerTabId, FOnSpawnTab::CreateRaw(this, &FProfileVisualizerModule::SpawnProfileVisualizerTab))
			.SetDisplayName(NSLOCTEXT("ProfileVisualizerModule", "TabTitle", "Profile Data Visualizer"))
			.SetTooltipText(NSLOCTEXT("ProfileVisualizerModule", "TooltipText", "Open the Profile Data Visualizer tab."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory())
			.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Profiler.Tab"));
#endif
			
	}
	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ProfileVisualizerTabId);
#endif		
		::ShutdownProfileVisualizer();
	}
	virtual void DisplayProfileVisualizer(TSharedPtr< FVisualizerEvent > InProfileData, const TCHAR* InProfilerType, const FText& HeaderMessageText, const FLinearColor& HeaderMessageTextColor) override
	{
#if	WITH_EDITOR
		::DisplayProfileVisualizer( InProfileData, InProfilerType, HeaderMessageText, HeaderMessageTextColor );
#endif // WITH_EDITOR
	}

	TSharedRef<SDockTab> SpawnProfileVisualizerTab(const FSpawnTabArgs& Args)
	{
		TSharedPtr< FVisualizerEvent > InProfileData(new FVisualizerEvent(0., 0., 0., 0, "Dummy"));
		
		return MakeProfileVisualizerWindow(InProfileData, FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), FLinearColor::White, false);
	}

};
IMPLEMENT_MODULE(FProfileVisualizerModule, ProfileVisualizer);
