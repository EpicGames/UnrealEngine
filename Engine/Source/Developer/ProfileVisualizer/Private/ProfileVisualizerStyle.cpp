// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileVisualizerStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

TSharedPtr< FProfileVisualizerStyle::FStyle > FProfileVisualizerStyle::StyleInstance;

FProfileVisualizerStyle::FStyle::FStyle()
	: FSlateStyleSet("ProfileVisualizerStyle")
{

}

void FProfileVisualizerStyle::FStyle::Initialize()
{
#if WITH_EDITORONLY_DATA
	StyleInstance->SetContentRoot( FPaths::EngineContentDir() / TEXT("Editor/Slate") );
	StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleInstance->Set( "StatsHeader", new BOX_BRUSH( "Common/TableViewHeader", 4.f/32.f ) );
	StyleInstance->Set( "ProfileVisualizer.ContentAreaBrush", new BOX_BRUSH( "/Docking/TabContentArea", FMargin(4/16.0f) ) );
	StyleInstance->Set( "ProfileVisualizer.Background", new BOX_BRUSH( "Common/ProgressBar_Background", FMargin(5.f/12.f) ) );
	StyleInstance->Set( "ProfileVisualizer.Normal", new CORE_BOX_BRUSH( "Common/ProfileVisualizer_Normal", FMargin(5.f/12.f) ) );
	StyleInstance->Set( "ProfileVisualizer.Selected", new CORE_BOX_BRUSH( "Common/ProfileVisualizer_Selected", FMargin(5.f/12.f) ) );
	StyleInstance->Set( "ProfileVisualizer.Mono", new CORE_BOX_BRUSH( "Common/ProfileVisualizer_Mono", FMargin(5.f/12.f) ) );
	StyleInstance->Set( "ProfileVisualizer.BorderPadding", FVector2D(1,0) );
	StyleInstance->Set( "ProfileVisualizer.SortUp", new IMAGE_BRUSH( "Common/SortUpArrow", FVector2D(8,4) ) );
	StyleInstance->Set( "ProfileVisualizer.SortDown", new IMAGE_BRUSH( "Common/SortDownArrow", FVector2D(8,4) ) );
	StyleInstance->Set( "ProfileVisualizer.Home", new IMAGE_BRUSH( "Icons/Home16x16", FVector2D(16,16) ) );
	StyleInstance->Set( "ProfileVisualizer.ToParent", new IMAGE_BRUSH( "Icons/ToParent", FVector2D(16,16) ) );
	StyleInstance->Set( "ProfileVisualizer.ProgressBar.BorderPadding", FVector2D(1,0) );
	StyleInstance->Set( "ProfileVisualizer.MenuDropdown", new IMAGE_BRUSH( "Common/ComboArrow", FVector2D(8,8) ) );

	FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());
#endif
}


void FProfileVisualizerStyle::Initialize() 
{
	StyleInstance = MakeShareable( new FProfileVisualizerStyle::FStyle );
	StyleInstance->Initialize();
}

void FProfileVisualizerStyle::Shutdown()
{
	if (StyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}
}

TSharedPtr< ISlateStyle > FProfileVisualizerStyle::Get()
{
	return StyleInstance;
}
