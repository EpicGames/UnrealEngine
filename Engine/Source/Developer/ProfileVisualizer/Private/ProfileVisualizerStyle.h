// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

/**
 * Declares the Editor's visual style.
 */
class FProfileVisualizerStyle
{
public:
	
	class FStyle : public FSlateStyleSet
	{
	public:
		FStyle();

		void Initialize();
	};

	static void Initialize();

	static void Shutdown();

	static TSharedPtr<class ISlateStyle> Get();

	static TSharedPtr< FProfileVisualizerStyle::FStyle > StyleInstance;
};
