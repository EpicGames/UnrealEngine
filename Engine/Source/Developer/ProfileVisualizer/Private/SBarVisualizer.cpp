// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBarVisualizer.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSlider.h"
#include "SGraphBar.h"
#include "STimeline.h"
#include "ProfileVisualizerStyle.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBarVisualizer::Construct( const FArguments& InArgs )
{
	ZoomSliderValue = 0.0f;
	ScrollbarOffset = 0.0f;
	ProfileData = InArgs._ProfileData.Get();
	OnBarGraphSelectionChangedDelegate = InArgs._OnBarGraphSelectionChanged;
	OnBarGraphExpansionChangedDelegate = InArgs._OnBarGraphExpansionChanged;
	OnBarEventSelectionChangedDelegate = InArgs._OnBarEventSelectionChanged;
	OnBarGraphContextMenuDelegate = InArgs._OnBarGraphContextMenu;
	bSuppressBarGraphSelectionChangedDelegate = false;

	ViewMode = EVisualizerViewMode::Hierarchical;
	SelectedBarGraph = ProfileData;
	CreateDataView();

	// Drop down menu
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ViewMenuBuilder( bInShouldCloseWindowAfterMenuSelection, NULL );
	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SBarVisualizer::SetViewMode, EVisualizerViewMode::Hierarchical ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SBarVisualizer::CheckViewMode, EVisualizerViewMode::Hierarchical ) );

		ViewMenuBuilder.AddMenuEntry( NSLOCTEXT("SBarVisualizer", "Hierarchical", "Hierarchical"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SBarVisualizer::SetViewMode, EVisualizerViewMode::Flat ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SBarVisualizer::CheckViewMode, EVisualizerViewMode::Flat ) );

		ViewMenuBuilder.AddMenuEntry( NSLOCTEXT("SBarVisualizer", "Flat", "Flat"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	const FSlateBrush* HomeButtonBrush = FProfileVisualizerStyle::Get()->GetBrush( "ProfileVisualizer.Home" );
	const FSlateBrush* ToParentButtonBrush = FProfileVisualizerStyle::Get()->GetBrush( "ProfileVisualizer.ToParent" );

	this->ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot().AutoHeight() .Padding( 2 ) .VAlign( VAlign_Fill )
		[
			SNew( SBorder )
			.BorderImage( FProfileVisualizerStyle::Get()->GetBrush("StatsHeader") )
			.ForegroundColor( FProfileVisualizerStyle::Get()->GetSlateColor("DefaultForeground") )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot().AutoWidth() .Padding( 2 ) .HAlign( HAlign_Left )
				[
					SNew(SButton)
					.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
					.ForegroundColor( FSlateColor::UseForeground() )
					.ContentPadding(FMargin(0))
					.Visibility( this, &SBarVisualizer::GetToParentButtonVisibility )
					.OnClicked( this, &SBarVisualizer::OnToParentClicked )
					[
						SNew(SImage)
						.Image( ToParentButtonBrush )
					]
				]
				+SHorizontalBox::Slot().AutoWidth().Padding( 2 ).HAlign( HAlign_Left )
				[
					SNew(SButton)
					.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ForegroundColor( FSlateColor::UseForeground() )
					.ContentPadding(FMargin(0))
					.Visibility( this, &SBarVisualizer::GetHomeButtonVisibility )
					.OnClicked( this, &SBarVisualizer::OnHomeClicked )
					[
						SNew(SImage)
						.Image( HomeButtonBrush )
					]
				]
				+SHorizontalBox::Slot() .Padding( 2 ) .FillWidth( 20 ) .HAlign( HAlign_Fill )
				[
					SNew( STextBlock )
					.Text( this, &SBarVisualizer::GetSelectedCategoryName )					
				]
				+SHorizontalBox::Slot().AutoWidth() .HAlign( HAlign_Right ) .Padding( FMargin(1.0f, 2.0f, 1.0f, 2.0f) )
				[
					SNew( SComboButton )
					//.ToolTipText(NSLOCTEXT("PropertyEditor", "ResetToDefaultToolTip", "Reset to Default").ToString())
					.HasDownArrow( false )
					.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
					.ContentPadding(0)
					.ButtonContent()
					[
						SNew(SImage)
						.Image( FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.MenuDropdown") )
					]
					.MenuContent()
					[
						ViewMenuBuilder.MakeWidget()
					]
				]	
			]
		]
		+SVerticalBox::Slot() .Padding( 2 ) .FillHeight( 1 ) .VAlign( VAlign_Fill )
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot() .Padding( 2 ) .FillWidth( 1 ) .HAlign( HAlign_Fill )
			[
				// List of thread graphs
				SAssignNew( BarGraphsList, SListView< TSharedPtr< FVisualizerEvent > > )
				// Tell the list view where to get its source data
				.ListItemsSource( &ProfileDataView )
				// When the list view needs to generate a widget for some data item, use this method
				.OnGenerateRow( this, &SBarVisualizer::OnGenerateWidgetForList )
				// Single selection mode
				.SelectionMode( ESelectionMode::Single )
				// Selection changed callback
				.OnSelectionChanged( this, &SBarVisualizer::OnBarGraphSelectionChanged )
			]
		]
		+SVerticalBox::Slot().AutoHeight() .Padding( 2 ) .VAlign( VAlign_Fill )
		[
			SAssignNew( Timeline, STimeline )
			.MinValue(0.0f)
			.MaxValue(SelectedBarGraph->DurationMs)
		]
		+SVerticalBox::Slot().AutoHeight() .Padding( 2 ) .VAlign( VAlign_Fill )
		[
			SAssignNew( ScrollBar, SScrollBar )
			.Orientation( Orient_Horizontal )
			.OnUserScrolled( this, &SBarVisualizer::ScrollBar_OnUserScrolled )
		]
		+SVerticalBox::Slot().AutoHeight() .Padding( 2 ) .VAlign( VAlign_Fill )
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.Padding( 2 )
			.FillWidth( 1 )
			.HAlign( HAlign_Fill )
			[
				SAssignNew( ZoomLabel, STextBlock )
					.Text( this, &SBarVisualizer::GetZoomLabel )
			]
			+SHorizontalBox::Slot()
			.Padding( 2 )
			.FillWidth( 5 )
			.HAlign( HAlign_Fill )
			[
				SNew( SSlider )
				.Value( this, &SBarVisualizer::GetZoomValue )
				.OnValueChanged( this, &SBarVisualizer::OnSetZoomValue )
			]
		]
	];


	ScrollBar->SetState(0.0f, 1.0f);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SBarVisualizer::ScrollBar_OnUserScrolled( float InScrollOffsetFraction )
{
	if( ZoomSliderValue > 0.0f )
	{
		const float MaxOffset = GetMaxScrollOffsetFraction();
		const float MaxGraphOffset = GetMaxGraphOffset();
		InScrollOffsetFraction = FMath::Clamp( InScrollOffsetFraction, 0.0f, MaxOffset );
		float GraphOffset = -( InScrollOffsetFraction / MaxOffset ) * MaxGraphOffset;

		ScrollBar->SetState( InScrollOffsetFraction, 1.0f / GetZoom() );

		for( int32 Index = 0; Index < Graphs.Num(); Index++ )
		{
			Graphs[ Index ]->SetOffset( GraphOffset );
		}

		Timeline->SetOffset( GraphOffset );

		ScrollbarOffset = GraphOffset;
	}
}

FText SBarVisualizer::GetZoomLabel() const
{
	static const FNumberFormattingOptions ZoomFormatOptions = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(2)
		.SetMaximumFractionalDigits(2);
	return FText::Format( NSLOCTEXT("ProfileVisualizer", "ZoomLabelFmt", "Zoom: {0}x"), FText::AsNumber(GetZoom(), &ZoomFormatOptions) );
}

float SBarVisualizer::GetZoomValue() const
{
	return ZoomSliderValue;
}

void SBarVisualizer::OnSetZoomValue( float NewValue )
{
	const float PrevZoom = GetZoom();
	const float PrevVisibleRange = 1.0f / PrevZoom;
		
	ZoomSliderValue = NewValue;
	const float Zoom = GetZoom();

	float GraphOffset = 0.0f;
	float ScrollOffsetFraction = 0.0f;

	if( Graphs.Num() )
	{
		const float MaxOffset = GetMaxScrollOffsetFraction();
		const float MaxGraphOffset = GetMaxGraphOffset();
		const float PrevGraphOffset = -Graphs[ 0 ]->GetOffset();
		GraphOffset = FMath::Clamp( -Graphs[ 0 ]->GetOffset(), 0.0f, MaxGraphOffset );
			
		const float VisibleRange = 1.0f / GetZoom();
		const float PrevGraphCenterValue = PrevGraphOffset / PrevZoom + PrevVisibleRange * 0.5f;
		const float NewGraphCenterValue = GraphOffset / Zoom + VisibleRange * 0.5f;
		GraphOffset += ( PrevGraphCenterValue - NewGraphCenterValue ) * Zoom;
		GraphOffset = FMath::Clamp( GraphOffset, 0.0f, MaxGraphOffset );

		ScrollOffsetFraction = FMath::Clamp( MaxOffset * GraphOffset / MaxGraphOffset, 0.0f, MaxOffset );
	}

	ScrollBar->SetState( ScrollOffsetFraction, 1.0f / Zoom );

	for( int32 Index = 0; Index < Graphs.Num(); Index++ )
	{
		Graphs[ Index ]->SetZoom( Zoom );
		Graphs[ Index ]->SetOffset( -GraphOffset );
	}

	Timeline->SetZoom( Zoom );
	Timeline->SetOffset( -GraphOffset );

	ScrollbarOffset = -GraphOffset;
}

void SBarVisualizer::OnBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection, ESelectInfo::Type SelectInfo )
{
	if( Selection.IsValid() )
	{
		BarGraphsList->RequestListRefresh();
	}

	if( !bSuppressBarGraphSelectionChangedDelegate )
	{
		OnBarGraphSelectionChangedDelegate.ExecuteIfBound( Selection );
	}
}

void SBarVisualizer::ClearBarSelection( TSharedPtr< FVisualizerEvent > GraphEvents, TSharedPtr<FVisualizerEvent> Selection )
{
	for( int32 ChildIndex = 0; ChildIndex < GraphEvents->Children.Num(); ChildIndex++ )
	{
		// Don't clear selection on the event to be selected
		if ( Selection != GraphEvents->Children[ ChildIndex ] )
		{
			GraphEvents->Children[ ChildIndex ]->IsSelected = false;
		}

		ClearBarSelection( GraphEvents->Children[ ChildIndex ], Selection );
	}
}

void SBarVisualizer::OnBarEventSelectionChanged( TSharedPtr<FVisualizerEvent> Selection, ESelectInfo::Type SelectInfo, int32 BarId )
{
	HandleEventSelectionChanged( Selection );
	OnBarEventSelectionChangedDelegate.ExecuteIfBound( BarId, Selection );
}

TSharedPtr< FVisualizerEvent > SBarVisualizer::FindSelectedEventsParent( TArray< TSharedPtr< FVisualizerEvent > >& BarGraphs, TSharedPtr< FVisualizerEvent > Selection )
{
	for( int32 Index = 0; Index < BarGraphs.Num(); Index++ )
	{
		if( BarGraphs[ Index ]->Children.Contains( Selection ) )
		{
			return BarGraphs[ Index ];
		}
		
		TSharedPtr< FVisualizerEvent > SelectionParent = FindSelectedEventsParent( BarGraphs[ Index ]->Children, Selection );
		if ( SelectionParent.IsValid() )
		{
			return SelectionParent;
		}
	}

	return TSharedPtr< FVisualizerEvent >();
}

void SBarVisualizer::HandleEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	// Clear any selected events from the other bar graphs
	const int32 BarId = Selection.IsValid() ? Selection->Category : INDEX_NONE;
	int32 BarIndex = INDEX_NONE;

	TSharedPtr< FVisualizerEvent > Root = ProfileData;
	while ( Root->ParentEvent.IsValid() )
	{
		Root = Root->ParentEvent;
	}

	ClearBarSelection( Root, Selection );

	// Since we're changing the selection as a result of selection change we don't want to create an infinite loop
	bSuppressBarGraphSelectionChangedDelegate = true;

	BarGraphsList->ClearSelection();

	if( Selection.IsValid() )
	{
		// Check if one of the bars has been selected
		bool BarGraphFound = false;
		for( int32 Index = 0; Index < ProfileDataView.Num() && BarGraphFound == false; Index++ )
		{
			TSharedPtr< FVisualizerEvent > BarGraph = ProfileDataView[ Index ];
			if( BarGraph->EventName == Selection->EventName )
			{
				BarGraphsList->SetSelection( BarGraph );
				BarGraphsList->RequestScrollIntoView( BarGraph );
				BarGraphFound = true;
			}
		}
	}

	bSuppressBarGraphSelectionChangedDelegate = false;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SBarVisualizer::OnGenerateWidgetForList( TSharedPtr<FVisualizerEvent> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	TSharedPtr<SGraphBar> RowGraph = SNew( SGraphBar )
				.OnSelectionChanged( this, &SBarVisualizer::OnBarEventSelectionChanged, InItem->Category )
				.OnGeometryChanged( this, &SBarVisualizer::OnBarGeometryChanged );

	const double EventsStartTime = InItem->ParentEvent.IsValid() ? InItem->ParentEvent->Start : 0.0;
	const double EventsEndTime = InItem->ParentEvent.IsValid() ? InItem->ParentEvent->Duration : 1.0;
	RowGraph->SetEvents( InItem->Children, EventsStartTime, EventsEndTime );
	Graphs.Add( RowGraph );
	RowGraph->SetZoom( GetZoom() );
	RowGraph->SetOffset( ScrollbarOffset );

	TSharedPtr<SWidget> BarTitle;

	if( IsExpandable( InItem ) )
	{
		BarTitle = SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
				.ForegroundColor( FSlateColor::UseForeground() )
				.ContentPadding(FMargin(0))
				.OnClicked( this, &SBarVisualizer::ExpandBar, InItem )
				[
					SNew(SBorder) 
					.BorderImage( FCoreStyle::Get().GetBrush("NoBorder") )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(0)						
					[
						SNew(SImage)
						.Image( FCoreStyle::Get().GetBrush( "TreeArrow_Collapsed" ) )
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( STextBlock ).Text( FText::FromString(InItem->EventName) )
			];
	}
	else
	{
		BarTitle = SNew( STextBlock ).Text( FText::FromString(InItem->EventName) );
	}

	return SNew( STableRow< TSharedPtr< FVisualizerEvent > >, OwnerTable )
		[
			SNew( SBorder )
			.Padding(0)
			.BorderImage( FCoreStyle::Get().GetBrush("NoBorder") )
			// Handle right-click event for context menu
			.OnMouseButtonDown( this, &SBarVisualizer::OnBarRightClicked, InItem )
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					BarTitle.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					RowGraph.ToSharedRef()
				]
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

/** Checks if the selected event has children with children */
bool SBarVisualizer::IsExpandable( TSharedPtr< FVisualizerEvent > InEvent )
{
	for( int32 ChildIndex = 0; ChildIndex < InEvent->Children.Num(); ChildIndex++ )
	{
		if( InEvent->Children[ ChildIndex ]->Children.Num() > 0 )
		{
			return true;
		}
	}
	return false;
}

void SBarVisualizer::OnGetChildrenForList( TSharedPtr<FVisualizerEvent> InItem, TArray<TSharedPtr<FVisualizerEvent> >& OutChildren )
{
	OutChildren = InItem->Children;
}

FReply SBarVisualizer::OnBarRightClicked( const FGeometry& BarGeometry, const FPointerEvent& MouseEvent, TSharedPtr<FVisualizerEvent> Selection )
{
	if( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton  )
	{
		if(OnBarGraphContextMenuDelegate.IsBound())
		{
			// Forward the event to the visualizer main frame
			// Disabled for now, may be useful in the future
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SBarVisualizer::CreateFlattenedData( TSharedPtr< FVisualizerEvent > InData, TArray< TSharedPtr< FVisualizerEvent > >& FlattenedData )
{
	for( int32 EventIndex = 0; EventIndex < InData->Children.Num(); EventIndex++ )
	{
		TSharedPtr< FVisualizerEvent > Event = InData->Children[ EventIndex ];
		TSharedPtr< FVisualizerEvent > FlattendedDataSet;

		if( Event->Category >= FlattenedData.Num() || FlattenedData[ Event->Category ].IsValid() == false )
		{
			// Fill with empty categories if necessary
			for( int32 CategoryIndex = FlattenedData.Num(); CategoryIndex <= Event->Category; CategoryIndex++ )
			{
				FlattenedData.Add( TSharedPtr< FVisualizerEvent >() );
			}

			// Create a separate data set for this category (thread)
			FlattendedDataSet = MakeShareable< FVisualizerEvent >( new FVisualizerEvent( 0.0, 1.0, 0.0, Event->Category, Event->EventName ) );
			FlattendedDataSet->Category = Event->Category;
			FlattenedData[ Event->Category ] = FlattendedDataSet;

			// Get the category name by looking for the first occurence of the category in the tree
			for( TSharedPtr< FVisualizerEvent > CategoryData = Event->ParentEvent; CategoryData.IsValid() && FlattendedDataSet->EventName.IsEmpty(); CategoryData = CategoryData->ParentEvent )
			{
				if( CategoryData->ParentEvent.IsValid() == false || CategoryData->ParentEvent->Category != Event->Category )
				{
					FlattendedDataSet->EventName = CategoryData->EventName;
				}
			}
		}
		else
		{
			FlattendedDataSet = FlattenedData[ Event->Category ];
		}

		// Fill with events
		if( InData->Children[ EventIndex ]->Children.Num() == 0 )
		{
			FlattendedDataSet->Children.Add( InData->Children[ EventIndex ] );
		}
	}

	for( int32 ChildIndex = 0; ChildIndex < InData->Children.Num(); ChildIndex++ )
	{
		CreateFlattenedData( InData->Children[ ChildIndex ], FlattenedData );
	}
}

void SBarVisualizer::CreateDataView()
{
	// Each time new data set is being displayed, clear all cached Bar Graphs
	Graphs.Empty();

	if( ViewMode == EVisualizerViewMode::Flat )
	{
		ProfileDataView.Empty();		
		TArray< TSharedPtr< FVisualizerEvent > > FlattenedBarGraphData;

		// At the top level each bar may represent a different subset of data (like a thread) so only flatten within one data set
		CreateFlattenedData( ProfileData, FlattenedBarGraphData );

		ProfileDataView.Append( FlattenedBarGraphData );
	}
	else
	{
		// Default to hierarchical. Choose the selected subset
		if( SelectedBarGraph.IsValid() == false )
		{
			ProfileDataView.Empty( 1 );
			ProfileDataView.Add( ProfileData );
		}
		else
		{
			ProfileDataView.Empty( SelectedBarGraph->Children.Num() );

			// Get all leaf events into one bar graph
			TArray< TSharedPtr< FVisualizerEvent > > LeafEvents; 
			for( int32 EventIndex = 0; EventIndex < SelectedBarGraph->Children.Num(); EventIndex++ )
			{
				TSharedPtr< FVisualizerEvent > Event = SelectedBarGraph->Children[ EventIndex ];
				if( Event->Children.Num() == 0 )
				{
					LeafEvents.Add( Event );
				}
				else
				{
					ProfileDataView.Add( Event );
				}
			}

			if( LeafEvents.Num() > 0 )
			{
				TSharedPtr< FVisualizerEvent > LeafEventsBarGraph( new FVisualizerEvent( SelectedBarGraph->Start, SelectedBarGraph->Duration, SelectedBarGraph->DurationMs, SelectedBarGraph->Category, SelectedBarGraph->EventName + TEXT(" Leaf Events") ) );
				LeafEventsBarGraph->ParentEvent = SelectedBarGraph;
				LeafEventsBarGraph->Children = LeafEvents;

				ProfileDataView.Add( LeafEventsBarGraph );
			}
		}
	}
}

void SBarVisualizer::SetViewMode( EVisualizerViewMode::Type InMode )
{	
	ViewMode = InMode;

	SelectedBarGraph = ProfileData;
	CreateDataView();

	BarGraphsList->RequestListRefresh();

	OnBarGraphExpansionChangedDelegate.ExecuteIfBound( SelectedBarGraph );
}

FReply SBarVisualizer::OnToParentClicked()
{
	if( SelectedBarGraph.IsValid() && SelectedBarGraph->ParentEvent.IsValid() )
	{
		SelectedBarGraph = SelectedBarGraph->ParentEvent;
		CreateDataView();

		BarGraphsList->RequestListRefresh();

		OnBarGraphExpansionChangedDelegate.ExecuteIfBound( SelectedBarGraph );

		AdjustTimeline( SelectedBarGraph );
	}
	return FReply::Handled();
}

FReply SBarVisualizer::OnHomeClicked()
{
	SelectedBarGraph = ProfileData;
	CreateDataView();
	BarGraphsList->RequestListRefresh();

	OnBarGraphExpansionChangedDelegate.ExecuteIfBound( SelectedBarGraph );
	AdjustTimeline( SelectedBarGraph );
	return FReply::Handled();
}

void SBarVisualizer::OnBarGeometryChanged( FGeometry Geometry )
{
	Timeline->SetDrawingGeometry( Geometry );
}

FText SBarVisualizer::GetSelectedCategoryName() const
{
	if( SelectedBarGraph.IsValid() )
	{
		FString EventName( SelectedBarGraph->EventName );

		for( TSharedPtr< FVisualizerEvent > BarGraph = SelectedBarGraph->ParentEvent; BarGraph.IsValid(); BarGraph = BarGraph->ParentEvent )
		{
			EventName = BarGraph->EventName + TEXT("\\") + EventName;
		}
		return FText::FromString(EventName);
	}
	else
	{
		return NSLOCTEXT("SBarVisualizer", "Frame", "Frame");
	}
}

EVisibility SBarVisualizer::GetHomeButtonVisibility() const
{
	if( SelectedBarGraph.IsValid() && SelectedBarGraph->ParentEvent.IsValid() && SelectedBarGraph->ParentEvent->ParentEvent.IsValid() )
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

EVisibility SBarVisualizer::GetToParentButtonVisibility() const
{
	if( SelectedBarGraph.IsValid() && SelectedBarGraph->ParentEvent.IsValid() )
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

FReply SBarVisualizer::ExpandBar( TSharedPtr<FVisualizerEvent> BarGraphEvents )
{
	if( BarGraphEvents.IsValid() && BarGraphEvents->Children.Num() > 0 )
	{
		SelectedBarGraph = BarGraphEvents;
		CreateDataView();
		BarGraphsList->RequestListRefresh();

		OnBarGraphExpansionChangedDelegate.ExecuteIfBound( SelectedBarGraph );

		// Set Timeline scale appropriate for the selected events
		AdjustTimeline( BarGraphEvents );
	}
	return FReply::Handled();
}

void SBarVisualizer::AdjustTimeline( TSharedPtr< FVisualizerEvent > InEvent )
{
	check( InEvent.IsValid() );

	const double TotalTimeMs = InEvent->DurationMs / InEvent->Duration;
	const double StartMs = InEvent->Start * TotalTimeMs;
	Timeline->SetMinMaxValues( StartMs, StartMs + InEvent->DurationMs );
}
