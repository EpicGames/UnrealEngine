// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEventsTree.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SEventsTree"

FName SEventsTree::NAME_NameColumn = FName(*NSLOCTEXT("ProfileVisualizer", "ColumnName", "Name").ToString());
FName SEventsTree::NAME_DurationColumn = FName( *NSLOCTEXT("ProfileVisualizer", "ColumnDuration", "Duration").ToString() );

void SEventItem::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	EventName = InArgs._EventName;
	EventDuration = InArgs._EventDuration;

	SMultiColumnTableRow< TSharedPtr< FVisualizerEvent > >::Construct( FSuperRowType::FArguments(), InOwnerTableView );	
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef< SWidget > SEventItem::GenerateWidgetForColumn( const FName& ColumnName )
{
	if( ColumnName == SEventsTree::NAME_NameColumn )
	{
		return  
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( STextBlock )
			.Text( FText::FromString(EventName) )
		];
	}
	else if( ColumnName == SEventsTree::NAME_DurationColumn )
	{
		return SNew( STextBlock )
			.Text( this, &SEventItem::GetDurationText );
	}
	else
	{
		return
		SNew( STextBlock )
		. Text(FText::Format( LOCTEXT("UnsupportedColumnFmt", "Unsupported Column: {0}"), FText::FromName(ColumnName) ) );
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SEventsTree::Construct( const FArguments& InArgs )
{
	DurationUnits = EVisualizerTimeUnits::Milliseconds;
	ViewMode = EVisualizerViewMode::Hierarchical;
	bSuppressSelectionChangedEvent = false;

	OnEventSelectionChangedDelegate = InArgs._OnEventSelectionChanged;
	ProfileData = InArgs._ProfileData.Get();

	// Duration column drop down menu
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bInShouldCloseWindowAfterMenuSelection, NULL );
	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetDurationUnits, EVisualizerTimeUnits::Microseconds ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckDurationUnits, EVisualizerTimeUnits::Microseconds ) );

		MenuBuilder.AddMenuEntry( LOCTEXT("Microseconds", "Microseconds"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetDurationUnits, EVisualizerTimeUnits::Milliseconds ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckDurationUnits, EVisualizerTimeUnits::Milliseconds ) );

		MenuBuilder.AddMenuEntry( LOCTEXT("Milliseconds", "Milliseconds"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetDurationUnits, EVisualizerTimeUnits::Seconds ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckDurationUnits, EVisualizerTimeUnits::Seconds ) );

		MenuBuilder.AddMenuEntry( LOCTEXT("Seconds", "Seconds"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	// Name column drop down menu
	FMenuBuilder NameMenuBuilder( bInShouldCloseWindowAfterMenuSelection, NULL );
	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetViewMode, EVisualizerViewMode::Hierarchical ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckViewMode, EVisualizerViewMode::Hierarchical ) );

		NameMenuBuilder.AddMenuEntry( LOCTEXT("Hierarchical", "Hierarchical"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetViewMode, EVisualizerViewMode::Flat ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckViewMode, EVisualizerViewMode::Flat ) );

		NameMenuBuilder.AddMenuEntry( LOCTEXT("Flat", "Flat"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetViewMode, EVisualizerViewMode::Coalesced ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckViewMode, EVisualizerViewMode::Coalesced ) );

		NameMenuBuilder.AddMenuEntry( LOCTEXT("Coalesced", "Coalesced"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SEventsTree::SetViewMode, EVisualizerViewMode::FlatCoalesced ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SEventsTree::CheckViewMode, EVisualizerViewMode::FlatCoalesced ) );

		NameMenuBuilder.AddMenuEntry( LOCTEXT("FlatCoalesced", "Flat Coalesced"), FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Check );
	}

	this->ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot().AutoHeight().Padding( 1.0f, 0.0f, 1.0f, 2.0f )
		[
			SNew( SSearchBox )
			.ToolTipText( NSLOCTEXT("ProfileVisualizer", "FilterSearchHint", "Type here to search events.") )
			.OnTextChanged( this, &SEventsTree::OnFilterTextChanged )
			.OnTextCommitted( this, &SEventsTree::OnFilterTextCommitted )
		]
		+SVerticalBox::Slot().Padding( 2 ).FillHeight( 1 ).VAlign( VAlign_Fill )
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot().Padding( 2 ).FillWidth( 1 ).HAlign( HAlign_Fill )
			[
				// List of all events for the selected thread
				SAssignNew( EventsListView, STreeView< TSharedPtr< FVisualizerEvent > > )
				// Tell the list view where to get its source data
				.TreeItemsSource( &SelectedEventsView )
				// When the list view needs to generate a widget for some data item, use this method
				.OnGenerateRow( this, &SEventsTree::OnGenerateWidgetForEventsList )
				// Given some DataItem, this is how we find out if it has any children and what they are.
				.OnGetChildren( this, &SEventsTree::OnGetChildrenForEventsList )
				// Selection mode
				.SelectionMode( ESelectionMode::Single )
				// Selection callback
				.OnSelectionChanged( this, &SEventsTree::OnEventSelectionChanged )
				.HeaderRow
				(
					SNew( SHeaderRow )
					+ SHeaderRow::Column( NAME_NameColumn )
					.DefaultLabel( NSLOCTEXT("ProfileVisualizer", "ColumnName", "Name") ) 
					.SortMode( TAttribute< EColumnSortMode::Type >::Create( TAttribute< EColumnSortMode::Type >::FGetter::CreateSP( this, &SEventsTree::GetColumnSortMode, NAME_NameColumn ) ) )
					.OnSort( FOnSortModeChanged::CreateSP( this, &SEventsTree::OnColumnSortModeChanged ) )
					.FillWidth( 1.0f )
					.MenuContent()
					[
						NameMenuBuilder.MakeWidget()
					]

					+ SHeaderRow::Column( NAME_DurationColumn )
					.DefaultLabel( this, &SEventsTree::GetDurationColumnTitle ) 
					.SortMode( TAttribute< EColumnSortMode::Type >::Create( TAttribute< EColumnSortMode::Type >::FGetter::CreateSP( this, &SEventsTree::GetColumnSortMode, NAME_DurationColumn ) ) )
					.OnSort( FOnSortModeChanged::CreateSP( this, &SEventsTree::OnColumnSortModeChanged ) )
					.FixedWidth( 128.0f )
					.MenuContent()
					[
						MenuBuilder.MakeWidget()
					]
				)
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

EColumnSortMode::Type SEventsTree::GetColumnSortMode( const FName ColumnId )
{
	if ( SortByColumn != ColumnId )
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

FText SEventsTree::GetDurationColumnTitle() const
{
	static const FText Units[] = { NSLOCTEXT("ProfileVisualizer", "microseconds", "microseconds"), NSLOCTEXT("ProfileVisualizer", "milliseconds", "ms"), NSLOCTEXT("ProfileVisualizer", "seconds", "s") };

	return FText::Format( NSLOCTEXT("ProfileVisualizer", "ColumnDurationValue", "Duration ({0})"), Units[ DurationUnits ] );
}

void ClearEventsSelection( TArray< TSharedPtr< FVisualizerEvent > >& Events )
{
	for( int32 Index = 0; Index < Events.Num(); ++Index )
	{
		TSharedPtr< FVisualizerEvent > CurrentEvent = Events[ Index ];
		CurrentEvent->IsSelected = false;

		// Clear recursively
		if( CurrentEvent->Children.Num() )
		{
			ClearEventsSelection( CurrentEvent->Children );
		}
	}
}

void SEventsTree::OnEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection, ESelectInfo::Type /*SelectInfo*/ )
{
	if( Selection.IsValid() )
	{
		ClearEventsSelection( SelectedEventsView );

		Selection->IsSelected = true;

		// Mirror the selection in the source events list
		TSharedPtr< FVisualizerEvent > MappedSelection = ViewToEventsMap.FindChecked( Selection );
		MappedSelection->IsSelected = true;

		if( !bSuppressSelectionChangedEvent )
		{
		OnEventSelectionChangedDelegate.ExecuteIfBound( MappedSelection );
	}
	}
	else if( !bSuppressSelectionChangedEvent )
	{
		OnEventSelectionChangedDelegate.ExecuteIfBound( Selection );
	}
}

TSharedRef< ITableRow > SEventsTree::OnGenerateWidgetForEventsList( TSharedPtr< FVisualizerEvent > InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew( SEventItem, OwnerTable )
			.EventName( InItem->EventName )
			.EventDuration( this, &SEventsTree::GetEventDuration, InItem->DurationMs );
}

double SEventsTree::GetEventDuration( double InDurationMs ) const
{
	switch( DurationUnits )
	{
	case EVisualizerTimeUnits::Microseconds:
		return InDurationMs * 1000.0;
	case EVisualizerTimeUnits::Milliseconds:
		return InDurationMs;
	case EVisualizerTimeUnits::Seconds:
		return InDurationMs * 0.001;
	}

	return InDurationMs;
}

void SEventsTree::SetDurationUnits( EVisualizerTimeUnits::Type InUnits )
{	
	DurationUnits = InUnits;
	EventsListView->RequestTreeRefresh();
}

void SEventsTree::SetViewMode( EVisualizerViewMode::Type InMode )
{	
	ViewMode = InMode;

	CreateSelectedEventsView();
	SortEventsList();

	EventsListView->RequestTreeRefresh();
}

void SEventsTree::OnGetChildrenForEventsList( TSharedPtr<FVisualizerEvent> InItem, TArray<TSharedPtr<FVisualizerEvent> >& OutChildren )
{
	OutChildren = InItem->Children;
}

void SEventsTree::OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode )
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	SortEventsList();
}

void SEventsTree::SortEventsList( TArray< TSharedPtr< FVisualizerEvent > >& Events )
{
	// Sort taking the current settings into account
	if( SortByColumn == NAME_NameColumn )
	{
		if( SortMode == EColumnSortMode::Ascending )
		{
			struct FCompareEventsByName
			{
				FORCEINLINE bool operator()( const TSharedPtr<FVisualizerEvent> A, const TSharedPtr<FVisualizerEvent> B ) const { return A->EventName < B->EventName; }
			};
			Events.Sort( FCompareEventsByName() );
		}
		else if( SortMode == EColumnSortMode::Descending )
		{
			struct FCompareEventsByNameDescending
			{
				FORCEINLINE bool operator()( const TSharedPtr<FVisualizerEvent> A, const TSharedPtr<FVisualizerEvent> B ) const { return B->EventName < A->EventName; }
			};
			Events.Sort( FCompareEventsByNameDescending() );
		}
	}
	else if( SortByColumn == NAME_DurationColumn )
	{
		if( SortMode == EColumnSortMode::Ascending )
		{
			struct FCompareEventsByDuration
			{
				FORCEINLINE bool operator()( const TSharedPtr<FVisualizerEvent> A, const TSharedPtr<FVisualizerEvent> B ) const { return A->DurationMs < B->DurationMs; }
			};
			Events.Sort( FCompareEventsByDuration() );
		}
		else if( SortMode == EColumnSortMode::Descending )
		{
			struct FCompareEventsByDurationDescending
			{
				FORCEINLINE bool operator()( const TSharedPtr<FVisualizerEvent> A, const TSharedPtr<FVisualizerEvent> B ) const { return B->DurationMs < A->DurationMs; }
			};
			Events.Sort( FCompareEventsByDurationDescending() );
		}
	}

	// Sort recursively
	for( int32 Index = 0; Index < Events.Num(); Index++ )
	{
		if( Events[ Index ]->Children.Num() )
		{
			SortEventsList( Events[ Index ]->Children );
		}
	}
}

void SEventsTree::SortEventsList( void )
{
	SortEventsList( SelectedEventsView );

	EventsListView->RequestTreeRefresh();

	RestoreEventSelection( SelectedEventsView );
}

bool SEventsTree::RestoreEventSelection( TArray< TSharedPtr< FVisualizerEvent > >& Events )
{
	// Search for the selected event
	for( int32 Index = 0; Index < Events.Num(); ++Index )
	{
		if( Events[ Index ]->IsSelected )
		{
			// Select it in the tree view widget
			EventsListView->ClearSelection();
			EventsListView->RequestTreeRefresh();
			EventsListView->SetSelection( Events[ Index ] );
			EventsListView->RequestScrollIntoView( Events[ Index ] );

			return true;
		}

		// Search recursively
		if( RestoreEventSelection( Events[ Index ]->Children ) )
		{
			return true;
		}
	}

	return false;
}

void SEventsTree::HandleBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	if ( Selection.IsValid() )
	{
		for( int32 EventIndex = 0; EventIndex < SelectedEvents.Num(); EventIndex++ )
		{
			TSharedPtr<FVisualizerEvent> Event = SelectedEvents[ EventIndex ];
			if( Selection->EventName == Event->EventName )
			{
				HandleBarEventSelectionChanged( SelectedThread->Category, Event );
				break;
			}
		}
	}
}

void SEventsTree::HandleBarGraphExpansionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	if ( Selection.IsValid() )
	{
		// We don't want to trigger selection changed event when the selection change is actually coming from the bar graph
		bSuppressSelectionChangedEvent = true;

		SelectedThread = Selection;
		SelectedEvents = Selection.Get()->Children;
		CreateSelectedEventsView();
		SortEventsList();

		bSuppressSelectionChangedEvent = false;
	}
}

void SEventsTree::HandleBarEventSelectionChanged( int32 Thread, TSharedPtr<FVisualizerEvent> Selection )
{
	const TSharedPtr< FVisualizerEvent >* MappedViewSelection = ViewToEventsMap.FindKey( Selection );
	if( MappedViewSelection )
	{
		TSharedPtr< FVisualizerEvent > ViewSelection = *MappedViewSelection;

		// Clear the current selection
		EventsListView->ClearSelection();
		EventsListView->RequestTreeRefresh();

		// Expand all parents so that it's visible
		for( TSharedPtr< FVisualizerEvent > ParentEvent = ViewSelection->ParentEvent; ParentEvent.IsValid(); ParentEvent = ParentEvent->ParentEvent )
		{
			EventsListView->SetItemExpansion( ParentEvent, true );
		}

		EventsListView->SetSelection( ViewSelection );

		EventsListView->RequestScrollIntoView( ViewSelection );
	}
	else
	{
		EventsListView->ClearSelection();
	}
}

FString SEventsTree::GetTabTitle() const
{
	if ( SelectedThread.IsValid() )
	{
		return SelectedThread->EventName;
	}
	else
	{
		return NSLOCTEXT("ProfileVisualizer", "EventsVisualizerName", "Empty Events List").ToString();
	}
}

int32 SEventsTree::CountEvents( TArray< TSharedPtr< FVisualizerEvent > >& Events )
{
	int32 Count = Events.Num();

	for( int32 Index = 0; Index < Events.Num(); Index++ )
	{
		Count += CountEvents( Events[ Index ]->Children );
	}

	return Count;
}

void SEventsTree::CreateSelectedEventsView()
{
	const int32 EventsCount = CountEvents( SelectedEvents );
	ViewToEventsMap.Empty( EventsCount );
	SelectedEventsView.Empty( SelectedEvents.Num() );

	// Create the selected events copy based on the current view mode
	if( ViewMode == EVisualizerViewMode::Hierarchical )
	{
		SelectedEventsView.Empty( SelectedEvents.Num() );

		for( int32 Index = 0; Index < SelectedEvents.Num(); Index++ )
		{
			if( FilterEvent( SelectedEvents[ Index ] ) )
			{
			TSharedPtr< FVisualizerEvent > EventCopy( CreateSelectedEventsViewRecursively( SelectedEvents[ Index ] ) );
				if( EventCopy.IsValid() )
				{
			SelectedEventsView.Add( EventCopy );
		}
	}
		}
	}
	else if( ViewMode == EVisualizerViewMode::Flat )
	{
		SelectedEventsView.Empty( EventsCount );

		for( int32 Index = 0; Index < SelectedEvents.Num(); Index++ )
		{
			CreateSelectedEventsViewRecursivelyAndFlatten( SelectedEvents[ Index ] );
		}
	}
	else if( ViewMode == EVisualizerViewMode::Coalesced )
	{
		SelectedEventsView.Empty( SelectedEvents.Num() );
		CreateSelectedEventsViewRecursivelyCoalesced( SelectedEvents, SelectedEventsView, TSharedPtr< FVisualizerEvent >() );
	}
	else if( ViewMode == EVisualizerViewMode::FlatCoalesced )
	{
		SelectedEventsView.Empty( EventsCount );
		CreateSelectedEventsViewRecursivelyFlatCoalesced( SelectedEvents );
	}
}

TSharedPtr< FVisualizerEvent > SEventsTree::CreateSelectedEventsViewRecursively( TSharedPtr< FVisualizerEvent > SourceEvent )
{
	TSharedPtr< FVisualizerEvent > EventCopy( new FVisualizerEvent( *SourceEvent ) );

	EventCopy->Children.Empty( SourceEvent->Children.Num() );
	for( int32 ChildIndex = 0; ChildIndex < SourceEvent->Children.Num(); ChildIndex++ )
	{
		if( FilterEvent( SourceEvent->Children[ ChildIndex ] ) )
		{
		TSharedPtr< FVisualizerEvent > ChildCopy = CreateSelectedEventsViewRecursively( SourceEvent->Children[ ChildIndex ] );
			if( ChildCopy.IsValid() )
			{
		ChildCopy->ParentEvent = EventCopy;
		EventCopy->Children.Add( ChildCopy );
	}
		}
	}

	// Add this event because it's a leaf or has valid children
	if( EventCopy->Children.Num() > 0 || SourceEvent->Children.Num() == 0 )
	{
		ViewToEventsMap.Add( EventCopy, SourceEvent );
	}
	else
	{
		EventCopy.Reset();
	}

	return EventCopy;
}

void SEventsTree::CreateSelectedEventsViewRecursivelyAndFlatten( TSharedPtr< FVisualizerEvent > SourceEvent )
{
	// Collect only children and store them directly into SelectedEventsView
	if ( FilterEvent( SourceEvent ) && SourceEvent->Children.Num() == 0 )
	{
		TSharedPtr< FVisualizerEvent > EventCopy( new FVisualizerEvent( *SourceEvent ) );
		EventCopy->ParentEvent.Reset();
		ViewToEventsMap.Add( EventCopy, SourceEvent );
		SelectedEventsView.Add( EventCopy );
	}
	
	for( int32 ChildIndex = 0; ChildIndex < SourceEvent->Children.Num(); ChildIndex++ )
	{
		CreateSelectedEventsViewRecursivelyAndFlatten( SourceEvent->Children[ ChildIndex ] );
	}
}

void SEventsTree::CreateSelectedEventsViewRecursivelyCoalesced( TArray< TSharedPtr< FVisualizerEvent > >& SourceEvents, TArray< TSharedPtr< FVisualizerEvent > >& CopiedEvents, TSharedPtr< FVisualizerEvent > InParent )
{
	for( int32 SourceIndex = 0; SourceIndex < SourceEvents.Num(); SourceIndex++ )
	{
		TSharedPtr< FVisualizerEvent > SourceEvent = SourceEvents[ SourceIndex ];
		if( FilterEvent( SourceEvent ) )
		{
		if( SourceEvent->Children.Num() == 0 )
		{
			// Check if a child with the same name has already been added to the copied event
			bool bEventExists = false;
			for( int32 CopiedEventIndex = 0; CopiedEventIndex < CopiedEvents.Num(); CopiedEventIndex++ )
			{
				if( CopiedEvents[ CopiedEventIndex ]->EventName == SourceEvent->EventName )
				{
					bEventExists = true;
					break;
				}
			}

			if( !bEventExists )
			{
				TSharedPtr< FVisualizerEvent > EventCopy( new FVisualizerEvent( *SourceEvent ) );
				EventCopy->ParentEvent = InParent;

				ViewToEventsMap.Add( EventCopy, SourceEvent );
				CopiedEvents.Add( EventCopy );				

				// Find other children with the same name and add their time to the copied one
				for( int32 OtherIndex = SourceIndex + 1; OtherIndex < SourceEvents.Num(); OtherIndex++ )
				{
					TSharedPtr< FVisualizerEvent > OtherEvent = SourceEvents[ OtherIndex ];
					if( OtherEvent->Children.Num() == 0 && OtherEvent->EventName == SourceEvent->EventName )
					{
						EventCopy->DurationMs += OtherEvent->DurationMs;
					}
				}
			}
		}
		else
		{
			TSharedPtr< FVisualizerEvent > EventCopy( new FVisualizerEvent( *SourceEvent ) );
			EventCopy->Children.Empty( SourceEvent->Children.Num() );
			EventCopy->ParentEvent = InParent;

				CreateSelectedEventsViewRecursivelyCoalesced( SourceEvent->Children, EventCopy->Children, EventCopy );

				// Only add this event if its children haven't been filtered
				if( EventCopy->Children.Num() )
				{
			ViewToEventsMap.Add( EventCopy, SourceEvent );
			CopiedEvents.Add( EventCopy );
				}
			}
		}
	}
}

void SEventsTree::CreateSelectedEventsViewRecursivelyFlatCoalesced( TArray< TSharedPtr< FVisualizerEvent > >& SourceEvents )
{
	for( int32 SourceIndex = 0; SourceIndex < SourceEvents.Num(); SourceIndex++ )
	{
		TSharedPtr< FVisualizerEvent > SourceEvent = SourceEvents[ SourceIndex ];
		if( FilterEvent( SourceEvent ) )
		{
			if( SourceEvent->Children.Num() == 0 )
			{
				// Check if a child with the same name has already been added to the copied event
				bool bEventExists = false;
				for( int32 CopiedEventIndex = 0; CopiedEventIndex < SelectedEventsView.Num(); CopiedEventIndex++ )
				{
					if( SelectedEventsView[ CopiedEventIndex ]->EventName == SourceEvent->EventName )
					{
						bEventExists = true;
						break;
					}
				}
  
				if( !bEventExists )
				{
					TSharedPtr< FVisualizerEvent > EventCopy( new FVisualizerEvent( *SourceEvent ) );
					EventCopy->ParentEvent.Reset();
  
					ViewToEventsMap.Add( EventCopy, SourceEvent );
					SelectedEventsView.Add( EventCopy );				
  
					// Find other children with the same name and add their time to the copied one
					for( int32 OtherIndex = SourceIndex + 1; OtherIndex < SourceEvents.Num(); OtherIndex++ )
					{
						TSharedPtr< FVisualizerEvent > OtherEvent = SourceEvents[ OtherIndex ];
						if( OtherEvent->Children.Num() == 0 && OtherEvent->EventName == SourceEvent->EventName )
						{
							EventCopy->DurationMs += OtherEvent->DurationMs;
						}
					}
				}
			}
			else
			{
				CreateSelectedEventsViewRecursivelyFlatCoalesced( SourceEvent->Children );
			}
		}
	}
}

void SEventsTree::OnFilterTextChanged( const FText& InFilterText )
{
	FilterText = InFilterText.ToString();
	CreateSelectedEventsView();
	EventsListView->RequestTreeRefresh();
}

void SEventsTree::OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type /*CommitInfo*/ )
{
}

#undef LOCTEXT_NAMESPACE
