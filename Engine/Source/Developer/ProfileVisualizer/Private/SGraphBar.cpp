// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphBar.h"
#include "Rendering/DrawElements.h"
#include "ProfileVisualizerStyle.h"

FColor SGraphBar::ColorPalette[] = {
	FColor( 0xff00A480 ),
	FColor( 0xff62E200 ),
	FColor( 0xff8F04A8 ),
	FColor( 0xff1F7B67 ),
	FColor( 0xff62AA2A ),
	FColor( 0xff70227E ),
	FColor( 0xff006B53 ),
	FColor( 0xff409300 ),
	FColor( 0xff5D016D ),
	FColor( 0xff34D2AF ),
	FColor( 0xff8BF13C ),
	FColor( 0xffBC38D3 ),
	FColor( 0xff5ED2B8 ),
	FColor( 0xffA6F16C ),
	FColor( 0xffC262D3 ),
	FColor( 0xff0F4FA8 ),
	FColor( 0xff00AE68 ),
	FColor( 0xffDC0055 ),
	FColor( 0xff284C7E ),
	FColor( 0xff21825B ),
	FColor( 0xffA52959 ),
	FColor( 0xff05316D ),
	FColor( 0xff007143 ),
	FColor( 0xff8F0037 ),
	FColor( 0xff4380D3 ),
	FColor( 0xff36D695 ),
	FColor( 0xffEE3B80 ),
	FColor( 0xff6996D3 ),
	FColor( 0xff60D6A7 ),
	FColor( 0xffEE6B9E )
};

void SGraphBar::Construct( const FArguments& InArgs )
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnGeometryChanged = InArgs._OnGeometryChanged;

	BackgroundImage = FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.Background");
	FillImage = FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.Mono");
	SelectedImage = FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.Selected");

	LastHoveredEvent = INDEX_NONE;
	Zoom = 1.0f;
	Offset = 0.0f;
	StartTime = 0.0;
	TotalTime = 1.0;
}

int32 SGraphBar::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// Used to track the layer ID we will return.
	int32 RetLayerId = LayerId;

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		
	const FLinearColor ColorAndOpacitySRGB = InWidgetStyle.GetColorAndOpacityTint();
	static const FLinearColor SelectedBarColor(FLinearColor::White);

	// Paint inside the border only. 
	const FVector2D BorderPadding = FProfileVisualizerStyle::Get()->GetVector("ProfileVisualizer.ProgressBar.BorderPadding");

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		RetLayerId++,
		AllottedGeometry.ToPaintGeometry(),
		BackgroundImage,
		DrawEffects,
		ColorAndOpacitySRGB
	);	
	
	// Draw all bars
	for( int32 EventIndex = 0; EventIndex < Events.Num(); ++EventIndex )
	{
		TSharedPtr< FVisualizerEvent > Event = Events[ EventIndex ];
		float StartX, EndX;
		if( CalculateEventGeometry( Event.Get(), AllottedGeometry, StartX, EndX ) )
		{
			// Draw Event bar
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				RetLayerId++,
				AllottedGeometry.ToPaintGeometry(
					FVector2f( EndX - StartX, AllottedGeometry.GetLocalSize().Y ),
					FSlateLayoutTransform(FVector2f( StartX, 0.0f ))),
				Event->IsSelected ? SelectedImage : FillImage,
				DrawEffects,
				Event->IsSelected ? SelectedBarColor : ColorPalette[Event->ColorIndex % (sizeof(ColorPalette) / sizeof(ColorPalette[0]))]			
				);
		}
	}

	return RetLayerId - 1;
}

void SGraphBar::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( OnGeometryChanged.IsBound() )
	{
		if( AllottedGeometry != this->LastGeometry )
		{
			OnGeometryChanged.ExecuteIfBound( AllottedGeometry );
			LastGeometry = AllottedGeometry;
		}
	}
}

FReply SGraphBar::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		// Translate click position from absolute to graph space
		const float ClickX = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X;//( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X / MyGeometry.Size.X ) / Zoom - Offset / Zoom;
		TSharedPtr< FVisualizerEvent > SelectedEvent;

		// Go through all events and check if at least one has been clicked
		int32 EventIndex = INDEX_NONE;
		for( EventIndex = 0; EventIndex < Events.Num(); EventIndex++ )
		{
			TSharedPtr< FVisualizerEvent > Event = Events[ EventIndex ];
			float StartX, EndX;
			if( CalculateEventGeometry( Event.Get(), MyGeometry, StartX, EndX ) )
			{
				if( ClickX >= StartX && ClickX < EndX )
				{
					// An event has been clicked, abort
					SelectedEvent = Event;
					Event->IsSelected = true;
					break;
				}
				else
				{
					Event->IsSelected = false;
				}
			}
		}

		// Iterate over all the remaining events and mark them as not selected
		for( EventIndex = EventIndex + 1; EventIndex < Events.Num(); EventIndex++ )
		{
			Events[ EventIndex ]->IsSelected = false;
		}

		// Execute OnSelectionChanged delegate
		if( OnSelectionChanged.IsBound() )
		{
			OnSelectionChanged.ExecuteIfBound( SelectedEvent, ESelectInfo::OnMouseClick );
		}

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}


/**
	* The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	*
	* @param MyGeometry The Geometry of the widget receiving the event
	* @param MouseEvent Information about the input event
	*
	* @return Whether the event was handled along with possible requests for the system to take action.
	*/
FReply SGraphBar::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const float HoverX = ( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X / MyGeometry.GetLocalSize().X ) / Zoom - Offset / Zoom;

	int32 HoveredEventIndex = INDEX_NONE;

	for( int32 EventIndex = 0; EventIndex < Events.Num(); EventIndex++ )
	{
		TSharedPtr< FVisualizerEvent > Event = Events[ EventIndex ];

		if( HoverX >= Event->Start && HoverX < ( Event->Start + Event->Duration ) )
		{
			HoveredEventIndex = EventIndex;
			break;
		}
	}

	if( HoveredEventIndex != LastHoveredEvent )
	{
		if( HoveredEventIndex != INDEX_NONE )
		{
			this->SetToolTipText( FText::FromString(Events[HoveredEventIndex]->EventName) );
		}
		else
		{
			this->SetToolTipText( FText::GetEmpty() );
		}
		LastHoveredEvent = HoveredEventIndex;
	}

	return SLeafWidget::OnMouseMove( MyGeometry, MouseEvent );
}

FVector2D SGraphBar::ComputeDesiredSize( float ) const
{
	return FVector2D( 8.0f, 16.0f );
}

void SGraphBar::SetEvents( const FVisualizerEventsArray& InEvents, double InStartTime, double InTotalTime )
{
	Events = InEvents;
	StartTime = InStartTime;
	TotalTime = InTotalTime;
}
