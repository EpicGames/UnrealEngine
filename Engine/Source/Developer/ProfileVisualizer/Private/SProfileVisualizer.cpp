// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProfileVisualizer.h"
#include "ProfileVisualizerModule.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SSplitter.h"
#include "ProfileVisualizerStyle.h"
#include "SBarVisualizer.h"
#include "SEventsTree.h"
#include "SProfileVisualizer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "HAL/FileManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#if PLATFORM_DESKTOP && WITH_EDITOR
#include "DesktopPlatformModule.h"
#endif

#include "Framework/Commands/GenericCommands.h"
	
void SProfileVisualizer::Construct(const FArguments& InArgs)
{
	ProfileData = InArgs._ProfileData;
	ProfilerType = InArgs._ProfilerType;
	HeaderMessageText = InArgs._HeaderMessageText;
	HeaderMessageTextColor = InArgs._HeaderMessageTextColor;

	const FSlateBrush* ContentAreaBrush = FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.ContentAreaBrush");
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SButton)
				.OnClicked(this, &SProfileVisualizer::OnLoadClicked)
				.Content()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("ProfileVisualizer", "Load", "Load"))
					.ToolTipText(NSLOCTEXT("ProfileVisualizer", "Load_GPUTooltip", "Load GPU profiling data"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SButton)
				.OnClicked(this, &SProfileVisualizer::OnSaveClicked)
				.Content()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("ProfileVisualizer", "Save", "Save"))
					.ToolTipText(NSLOCTEXT("ProfileVisualizer", "Save_GPUTooltip", "Save the GPU profiling data"))
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(ContentAreaBrush)
			[
				SNew(STextBlock)
				.Visibility( HeaderMessageText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible )
				.Text(HeaderMessageText)
				.ColorAndOpacity(HeaderMessageTextColor)
				.AutoWrapText(true)
			]
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew( MainSplitter, SSplitter )
			.Orientation( Orient_Vertical )
			+ SSplitter::Slot()
			.Value(1.0f)
			[
				SNew( SBorder )
				.Visibility( EVisibility::Visible )
				.BorderImage( ContentAreaBrush )
				[
					SAssignNew( BarVisualizer, SBarVisualizer )
					.ProfileData( ProfileData )
					.OnBarGraphSelectionChanged( this, &SProfileVisualizer::RouteBarGraphSelectionChanged )
					.OnBarGraphExpansionChanged( this, &SProfileVisualizer::RouteBarGraphExpansionChanged )
					.OnBarEventSelectionChanged( this, &SProfileVisualizer::RouteBarEventSelectionChanged )
					.OnBarGraphContextMenu( this, &SProfileVisualizer::OnBarGraphContextMenu )
				]
			]
			+ SSplitter::Slot()
			.Value(1.0f)
			[
				SNew( SBorder )
				.Visibility( EVisibility::Visible )
				.BorderImage( ContentAreaBrush )
				[
					SAssignNew( EventsTree, SEventsTree )
					.ProfileData( ProfileData )
					.OnEventSelectionChanged( this, &SProfileVisualizer::RouteEventSelectionChanged )
				]
			]
		]
	];

	// Attempt to choose initial data set to display in the tree view
	EventsTree->HandleBarGraphExpansionChanged( ProfileData );
}

TSharedRef< SWidget > SProfileVisualizer::MakeMainMenu()
{
	FMenuBarBuilder MenuBuilder( NULL );
	{
		// File
		MenuBuilder.AddPullDownMenu( 
			NSLOCTEXT("ProfileVisualizer", "FileMenu", "File"),
			NSLOCTEXT("ProfileVisualizer", "FileMenu_ToolTip", "Open the file menu"),
			FNewMenuDelegate::CreateSP( this, &SProfileVisualizer::FillFileMenu ) );

		// Apps
		MenuBuilder.AddPullDownMenu( 
			NSLOCTEXT("ProfileVisualizer", "AppMenu", "Window"),
			NSLOCTEXT("ProfileVisualizer", "AppMenu_ToolTip", "Open the summoning menu"),
			FNewMenuDelegate::CreateSP( this, &SProfileVisualizer::FillAppMenu ) );

		// Help
		MenuBuilder.AddPullDownMenu( 
			NSLOCTEXT("ProfileVisualizer", "HelpMenu", "Help"),
			NSLOCTEXT("ProfileVisualizer", "HelpMenu_ToolTip", "Open the help menu"),
			FNewMenuDelegate::CreateSP( this, &SProfileVisualizer::FillHelpMenu ) );
	}

	// Create the menu bar
	TSharedRef< SWidget > MenuBarWidget = MenuBuilder.MakeWidget();

	return MenuBarWidget;
}

void SProfileVisualizer::FillFileMenu( FMenuBuilder& MenuBuilder )
{
}

void SProfileVisualizer::FillAppMenu( FMenuBuilder& MenuBuilder )
{
}

void SProfileVisualizer::FillHelpMenu( FMenuBuilder& MenuBuilder )
{
}


void SProfileVisualizer::RouteEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	BarVisualizer->HandleEventSelectionChanged( Selection );
}

void SProfileVisualizer::RouteBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	EventsTree->HandleBarGraphSelectionChanged( Selection );
}

void SProfileVisualizer::RouteBarGraphExpansionChanged( TSharedPtr< FVisualizerEvent > Selection )
{
	EventsTree->HandleBarGraphExpansionChanged( Selection );
}

void SProfileVisualizer::RouteBarEventSelectionChanged( int32 Thread, TSharedPtr<FVisualizerEvent> Selection )
{
	EventsTree->HandleBarEventSelectionChanged( Thread, Selection );
}

void SProfileVisualizer::OnBarGraphContextMenu( TSharedPtr< FVisualizerEvent > Selection, const FPointerEvent& InputEvent )
{
	SelectedBarGraph = Selection;

	FWidgetPath WidgetPath = InputEvent.GetEventPath() != nullptr ? *InputEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MakeBarVisualizerContextMenu(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
}

TSharedRef<SWidget> SProfileVisualizer::MakeBarVisualizerContextMenu()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder( bCloseAfterSelection, NULL );
	{
		FUIAction Action( FExecuteAction::CreateSP( this, &SProfileVisualizer::ShowGraphBarInEventsWindow, (int32)INDEX_NONE ) );
		MenuBuilder.AddMenuEntry( NSLOCTEXT("ProfileVisualizer", "GraphBarShowInNew", "Show in New Events Window"), FText::GetEmpty(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Button );
	}
		
	return MenuBuilder.MakeWidget();
}

void SProfileVisualizer::ShowGraphBarInEventsWindow( int32 WindowIndex )
{
	EventsTree->HandleBarGraphExpansionChanged( SelectedBarGraph );
}

FReply SProfileVisualizer::OnSaveClicked()
{
#if PLATFORM_DESKTOP && WITH_EDITOR
	// Message to display on completion
	FText Message;

	FString ProfileFilename = FPaths::ProjectLogDir();
	ProfileFilename /= TEXT("profileViz");
	ProfileFilename /= FString::Printf(
		TEXT("%s-%i-%s.profViz"),
		FApp::GetProjectName(),
		FEngineVersion::Current().GetChangelist(),
		*FDateTime::Now().ToString());

	FArchive* ProfileFile = IFileManager::Get().CreateFileWriter(*ProfileFilename);

	if (ProfileFile)
	{
		FVisualizerEvent::SaveVisualizerEventRecursively(ProfileFile, ProfileData);

		// Close and delete archive.
		ProfileFile->Close();
		delete ProfileFile;
		ProfileFile = NULL;

		Message = NSLOCTEXT("ProfileVisualizer", "ExportMessage", "Wrote profile data to file");
	}
	else
	{
		Message = NSLOCTEXT("ProfileVisualizer", "FailedExportMessage", "Could not write profile data to file");
	}

	struct Local
	{
		static void NavigateToExportedFile(FString InGPUFilename, bool bInSuccessful)
		{
			InGPUFilename = FPaths::ConvertRelativePathToFull(InGPUFilename);
			if (bInSuccessful)
			{
				FPlatformProcess::LaunchFileInDefaultExternalApplication(*InGPUFilename);
			}
			else
			{
				FPlatformProcess::ExploreFolder(*FPaths::GetPath(InGPUFilename));
			}
		}
	};

	FNotificationInfo Info(Message);
	Info.Hyperlink = FSimpleDelegate::CreateStatic(&Local::NavigateToExportedFile, ProfileFilename, false);
	Info.HyperlinkText = FText::FromString(ProfileFilename);
	Info.bUseLargeFont = false;
	Info.bFireAndForget = true;
	Info.ExpireDuration = 8.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
#endif //#if PLATFORM_DESKTOP && WITH_EDITOR

	return FReply::Handled();

}

FReply SProfileVisualizer::OnLoadClicked()
{
#if PLATFORM_DESKTOP && WITH_EDITOR
	// Prompt the user for the Filename
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		TArray<FString> OpenFilenames;
		bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("UnrealEd", "Load", "Load Profile data").ToString(),
			TEXT(""),
			TEXT(""),
			TEXT("Profile data (*.profViz) | *.profViz"),
			EFileDialogFlags::None,
			OpenFilenames);

		if (bOpened)
		{
			FArchive* ProfileFile = IFileManager::Get().CreateFileReader(*OpenFilenames[0]);

			if (ProfileFile)
			{
				TSharedPtr< FVisualizerEvent > InVisualizerData;
				InVisualizerData = FVisualizerEvent::LoadVisualizerEvent(ProfileFile);
								
				static FName ProfileVisualizerModule(TEXT("ProfileVisualizer"));
				if (FModuleManager::Get().IsModuleLoaded(ProfileVisualizerModule))
				{
					IProfileVisualizerModule& ProfileVisualizer = FModuleManager::GetModuleChecked<IProfileVisualizerModule>(ProfileVisualizerModule);

					FText LoadedFileName = FText::AsCultureInvariant(ProfileFile->GetArchiveName());
					ProfileVisualizer.DisplayProfileVisualizer(InVisualizerData, TEXT("Profile Data"), LoadedFileName);
				}

				// Close and delete archive.
				ProfileFile->Close();
				delete ProfileFile;
				ProfileFile = NULL;
			}
		}
	}
#endif//#if PLATFORM_DESKTOP && WITH_EDITOR

	return FReply::Handled();

}
