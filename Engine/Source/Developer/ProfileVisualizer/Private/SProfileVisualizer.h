// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "VisualizerEvents.h"

class FMenuBuilder;
class SBarVisualizer;
class SEventsTree;
class SSplitter;

class SProfileVisualizer : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProfileVisualizer)
		: _LayoutString()
		, _ProfileData()
		, _ProfilerType()
		{}

		SLATE_ARGUMENT( FString, LayoutString )

		SLATE_ARGUMENT( TSharedPtr< FVisualizerEvent >, ProfileData )

		SLATE_ARGUMENT( FText, ProfilerType )

		SLATE_ARGUMENT( FText, HeaderMessageText )

		SLATE_ARGUMENT( FLinearColor, HeaderMessageTextColor )

	SLATE_END_ARGS()

	
	/**
	 * Constructs this widget
	 *
	 * @param InArgs    Declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

private:

	/** Construct main menu */
	TSharedRef< SWidget > MakeMainMenu();

	/** Fills the contents of 'File' menu */
	void FillFileMenu( FMenuBuilder& MenuBuilder );

	/** Fills the contents of 'Window' menu */
	void FillAppMenu( FMenuBuilder& MenuBuilder );

	/** Fills the contents of 'Help' menu */
	void FillHelpMenu( FMenuBuilder& MenuBuilder );

	/** Routes events from events tree tab to bar visualizer tabs */
	void RouteEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection );

	/** Routes events from bar visualizer tab to events tree tabs */
	void RouteBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection );

	/** Routes events from bar visualizer tab to events tree tabs */
	void RouteBarGraphExpansionChanged( TSharedPtr< FVisualizerEvent > Selection );

	/** Routes events from bar visualizer tab to events tree tabs */
	void RouteBarEventSelectionChanged( int32 Thread, TSharedPtr<FVisualizerEvent> Selection );

	/** Opens a context menu when a bar is right clicked */
	void OnBarGraphContextMenu( TSharedPtr< FVisualizerEvent > Selection, const FPointerEvent& InputEvent );

	/** Creates bar visualizer context menu for opening new events tree tabs */
	TSharedRef<SWidget> MakeBarVisualizerContextMenu();

	/** Handles creating a new events tree tab from the bar visualizer context menu */
	void ShowGraphBarInEventsWindow( int32 WindowIndex );

	/** Pointer to the main dock area of this widget */
	TSharedPtr< SSplitter > MainSplitter;

	/** Pointer to the bar visualizer */
	TSharedPtr< SBarVisualizer > BarVisualizer;

	/** Pointer to the events visualizer */
	TSharedPtr< SEventsTree > EventsTree;

	/** Profiler results */
	TSharedPtr< FVisualizerEvent > ProfileData;

	/** Currently selected bar graph */
	TSharedPtr< FVisualizerEvent > SelectedBarGraph;

	/** Profiler name */
	FText ProfilerType;

	/** Optional header message to display at the top of the profile window */
	FText HeaderMessageText;
	 
	/** Optional header message text color */
	FLinearColor HeaderMessageTextColor;

	/** We just clicked the save button */
	FReply OnSaveClicked();

	/** We just clicked the load button */
	FReply OnLoadClicked();
};

