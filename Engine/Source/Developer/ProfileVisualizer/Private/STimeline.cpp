// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimeline.h"
#include "Misc/Paths.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontMeasure.h"
#include "Styling/CoreStyle.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"
#include "ProfileVisualizerStyle.h"

void STimeline::Construct( const FArguments& InArgs )
{
	MinValue = InArgs._MinValue;
	MaxValue = InArgs._MaxValue;
	FixedLabelSpacing = InArgs._FixedLabelSpacing;

	BackgroundImage = FProfileVisualizerStyle::Get()->GetBrush("ProfileVisualizer.Background");

	Zoom = 1.0f;
	Offset = 0.0f;
}

int32 STimeline::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// Used to track the layer ID we will return.
	int32 RetLayerId = LayerId;
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		
	const FLinearColor ColorAndOpacitySRGB = InWidgetStyle.GetColorAndOpacityTint();
	static const FLinearColor SelectedBarColor(FLinearColor::White);

	// Paint inside the border only. 
	const FVector2D BorderPadding = FProfileVisualizerStyle::Get()->GetVector("ProfileVisualizer.ProgressBar.BorderPadding");
	
	const float OffsetX = DrawingOffsetX; // BorderPadding.X
	const float Width = DrawingGeometry.Size.X; // AllottedGeometry.Size.X - - 2.0f * BorderPadding.X

	FSlateFontInfo MyFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	// Create line points
	const float RoundedMax = FMath::CeilToInt( MaxValue );
	const float RoundedMin = FMath::FloorToInt( MinValue );
	const float TimeScale = MaxValue - MinValue;
	const int32 NumValues = FMath::FloorToInt( AllottedGeometry.Size.X * Zoom / FixedLabelSpacing );

	TArray< FVector2D > LinePoints;
	LinePoints.AddUninitialized( 2 );
	
	{
		LinePoints[0] = FVector2D( OffsetX, BorderPadding.Y + 1.0f );
		LinePoints[1] = FVector2D( OffsetX + Width, BorderPadding.Y + 1.0f );

		// Draw lines
		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
			);
	}

	const FVector2D TextDrawSize = FontMeasureService->Measure(TEXT("0.00"), MyFont);
	const float LineHeight = AllottedGeometry.Size.Y - BorderPadding.Y - TextDrawSize.Y - 2.0f;
	
	for( int32 LineIndex = 0; LineIndex <= NumValues; LineIndex++ )
	{
		const float NormalizedX = (float)LineIndex / NumValues; //-V609 
		const float LineX = Offset + NormalizedX * Zoom;
		if( LineX < 0.0f || LineX > 1.0f )
		{
			continue;
		}

		const float LineXPos =  OffsetX + Width * LineX;
		LinePoints[0] = FVector2D( LineXPos, BorderPadding.Y );
		LinePoints[1] = FVector2D( LineXPos, LineHeight );

		// Draw lines
		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
			);

		FString ValueText( FString::Printf( TEXT("%.2f"), MinValue + NormalizedX * TimeScale ) );
		FVector2D DrawSize = FontMeasureService->Measure(ValueText, MyFont);
		FVector2D TextPos( LineXPos - DrawSize.X * 0.5f, LineHeight );

		if( TextPos.X < 0.0f )
		{
			TextPos.X = 0.0f;
		}
		else if( (TextPos.X + DrawSize.X) > AllottedGeometry.Size.X )
		{
			TextPos.X = OffsetX + Width - DrawSize.X;
		}

		FSlateDrawElement::MakeText( 
			OutDrawElements,
			RetLayerId,
			AllottedGeometry.ToOffsetPaintGeometry( TextPos ),
			ValueText,
			MyFont,
			ESlateDrawEffect::None,
			FLinearColor::White
			);
	}

	// Always draw lines at the start and at the end of the timeline
	{
		LinePoints[0] = FVector2D( OffsetX, BorderPadding.Y );
		LinePoints[1] = FVector2D( OffsetX, LineHeight );

		// Draw lines
		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
			);
	}

	{
		LinePoints[0] = FVector2D( OffsetX + Width, BorderPadding.Y );
		LinePoints[1] = FVector2D( OffsetX + Width, LineHeight );

		// Draw lines
		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
			);
	}

	return RetLayerId - 1;
}

void STimeline::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	DrawingOffsetX = DrawingGeometry.AbsolutePosition.X - AllottedGeometry.AbsolutePosition.X;
}

FReply STimeline::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return SCompoundWidget::OnMouseButtonDown( MyGeometry, MouseEvent );
}

FReply STimeline::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return SCompoundWidget::OnMouseMove( MyGeometry, MouseEvent );
}

FVector2D STimeline::ComputeDesiredSize( float ) const
{
	return FVector2D( 8.0f, 8.0f );
}
