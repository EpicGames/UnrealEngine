// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualizerEvents.h"

#if defined( _WIN64 )
static_assert(sizeof(FVisualizerEvent) == 88, "FVisualizerEvent size has changed. Please update the save/load functions");
#endif

TSharedPtr< FVisualizerEvent > FVisualizerEvent::LoadVisualizerEventRecursively(FArchive *Ar, TSharedPtr< FVisualizerEvent > InParentEvent)
{
	double Start;
	double Duration;
	double DurationMs;
	int32 Category;
	FString EventName;
	bool IsSelected;

	*Ar << Start;
	*Ar << Duration;
	*Ar << DurationMs;
	*Ar << Category;
	*Ar << EventName;
	*Ar << IsSelected;

	TSharedPtr< FVisualizerEvent > VisualizerEvent(new FVisualizerEvent(Start, Duration,DurationMs,Category,EventName));
	VisualizerEvent->ParentEvent = InParentEvent;

	uint32 NumChildren;

	Ar->SerializeInt(NumChildren, MAX_uint32);
	VisualizerEvent->Children.Reserve(NumChildren);

	for (uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr< FVisualizerEvent > ChildEvent = LoadVisualizerEventRecursively(Ar, VisualizerEvent);
		VisualizerEvent->Children.Add(ChildEvent);
	}

	return VisualizerEvent;

}

TSharedPtr< FVisualizerEvent > FVisualizerEvent::LoadVisualizerEvent(FArchive *Ar)
{
	//FArchive Ar;
	// Assumption: InProfileData contains only one (root) element. Otherwise an extra FVisualizerEvent root event is required.
	TSharedPtr< FVisualizerEvent > DummyRoot;
	// Recursively create visualizer event data.
	TSharedPtr< FVisualizerEvent > StatEvents(LoadVisualizerEventRecursively( Ar, DummyRoot));
	return StatEvents;
}



void FVisualizerEvent::SaveVisualizerEventRecursively(FArchive *Ar, TSharedPtr< FVisualizerEvent > VisualizerEvent )
{
	//Write all simple members
	*Ar << VisualizerEvent->Start;
	*Ar << VisualizerEvent->Duration;
	*Ar << VisualizerEvent->DurationMs;
	*Ar << VisualizerEvent->Category;
	*Ar << VisualizerEvent->EventName;
	*Ar << VisualizerEvent->IsSelected;

	uint32 NumChildren;

	NumChildren = VisualizerEvent->Children.Num();
	Ar->SerializeInt(NumChildren, MAX_uint32);

	for(uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr< FVisualizerEvent > Child;

		Child = VisualizerEvent->Children[i];
		SaveVisualizerEventRecursively(Ar, Child);
	}
}


