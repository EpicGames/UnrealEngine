// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ProfileVisualizer : ModuleRules
{
	public ProfileVisualizer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
				
				"Engine",
				"InputCore"
			}
		);

		if ((Target.bCompileAgainstEditor && (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"WorkspaceMenuStructure",
					"DesktopPlatform"
				}
			);
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
