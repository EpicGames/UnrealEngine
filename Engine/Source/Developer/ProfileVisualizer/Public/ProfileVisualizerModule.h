// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "VisualizerEvents.h"

class IProfileVisualizerModule
	: public IModuleInterface
{
public:
	virtual void DisplayProfileVisualizer( TSharedPtr< FVisualizerEvent > InProfileData, const TCHAR* InProfilerType, const FText& HeaderMessageText = FText::GetEmpty(), const FLinearColor& HeaderMessageTextColor = FLinearColor::White ) = 0;
};
