// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "VisualizerEvents.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#define UE_API PROFILEVISUALIZER_API

class SGraphBar;
class SScrollBar;
class STextBlock;
class STimeline;

/**
 * Bars Visualizer. Contains a list of bars for each profiler category
 */
class SBarVisualizer : public SCompoundWidget
{
public:

	/** Delegate used when bar graph selection changes */
	DECLARE_DELEGATE_OneParam( FOnBarGraphSelectionChanged, TSharedPtr< FVisualizerEvent > );

	/** Delegate used when bar graph expansion changes */
	DECLARE_DELEGATE_OneParam( FOnBarGraphExpansionChanged, TSharedPtr< FVisualizerEvent > );

	/** Delegate used when a single event on the bar graph is selected */
	DECLARE_DELEGATE_TwoParams( FOnBarEventSelectionChanged, int32, TSharedPtr< FVisualizerEvent > );
	
	/** Delegate used when the user right-clicks on a bar graph */
	DECLARE_DELEGATE_TwoParams( FOnBarGraphContextMenu, TSharedPtr< FVisualizerEvent >, const FPointerEvent& );

	SLATE_BEGIN_ARGS( SBarVisualizer )
		: _ProfileData()
		, _OnBarGraphSelectionChanged()
		, _OnBarGraphExpansionChanged()
		, _OnBarEventSelectionChanged()
		, _OnBarGraphContextMenu()
		{}

		/** Profiler results */
		SLATE_ATTRIBUTE( TSharedPtr< FVisualizerEvent >, ProfileData )

		/** Callback triggered when bar graph selection changes */
		SLATE_EVENT( FOnBarGraphSelectionChanged, OnBarGraphSelectionChanged )

		/** Callback triggered when bar graph expansion changes */
		SLATE_EVENT( FOnBarGraphExpansionChanged, OnBarGraphExpansionChanged )

		/** Callback triggered when single event on the bar graph is selected */
		SLATE_EVENT( FOnBarEventSelectionChanged, OnBarEventSelectionChanged )

		/** Callback triggered when the user right-clicks on a bar graph */
		SLATE_EVENT( FOnBarGraphContextMenu, OnBarGraphContextMenu )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	/**
	 * Handles selection change in the events tree
	 *
	 * @param Selection selected event
	 */
	UE_API void HandleEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection );

protected:

	/**
	 * Gets the maximum scroll offset fraction value for the horizontal scrollbar
	 *
	 * @return Maximum scroll offset fraction
	 */
	float GetMaxScrollOffsetFraction() const
	{
		return 1.0f - 1.0f / GetZoom();
	}

	/**
	 * Gets the maximum graph offset value for the graph bars
	 *
	 * @return Maximum graph offset
	 */
	float GetMaxGraphOffset() const
	{
		return GetZoom() - 1.0f;
	}

	/**
	 * Gets the actual zoom level for the graph bars
	 *
	 * @return Zoom level
	 */
	float GetZoom() const
	{
		const float MinZoom = 1.0f;
		const float MaxZoom = 20.0f;
		return MinZoom + ZoomSliderValue * ( MaxZoom - MinZoom );
	}

	/**
	 * Callback for scrolling the horizontal scrollbar
	 *
	 * @param InScrollOffsetFraction Scrollbar offset fraction
	 */
	UE_API void ScrollBar_OnUserScrolled( float InScrollOffsetFraction );

	/**
	 * Constructs the zoom label string based on the current zoom level value.
	 *
	 * @return Zoom label text
	 */
	UE_API FText GetZoomLabel() const;

	/**
	 * Callback used to get the current zoom slider value.
	 *
	 * @return Zoom slider value
	 */
	UE_API float GetZoomValue() const;

	/**
	 * Callback used to handle zoom slider
	 *
	 * @param NewValue New Zoom slider value
	 */
	UE_API void OnSetZoomValue( float NewValue );

	/**
	 * Function called when the currently selected bar graph changes
	 *
	 * @param Selection Currently selected thread events
	 * @param SelectInfo Provides context on how the selection changed
	 */
	UE_API void OnBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection, ESelectInfo::Type SelectInfo );

	/**
	 * Function called when the user selects an event bar in the graph
	 *
	 * @param Selection Currently selected event
	 * @param SelectInfo Provides context on how the selection changed
	 */
	UE_API void OnBarEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection, ESelectInfo::Type SelectInfo, int32 BarId );

	/** 
	 * Generates SGraphBar widget for the threads list
	 *
	 * @param InItem Graph profile events
	 * @param OwnerTable Onwer Table
	 */
	UE_API TSharedRef<ITableRow> OnGenerateWidgetForList( TSharedPtr< FVisualizerEvent > InItem, const TSharedRef< STableViewBase >& OwnerTable );

	/** 
	 * Generates children for the specified tree view item
	 *
	 * @param InItem Graph profile events
	 * @param OutChildren child graphs
	 */
	UE_API void OnGetChildrenForList( TSharedPtr<FVisualizerEvent> InItem, TArray<TSharedPtr<FVisualizerEvent> >& OutChildren);

	/**
	 * Forwards right-click event to the visualizer main frame
	 *
	 * @param BarGeometry Graph bar geometry
	 * @param MouseEvent Current mouse event
	 * @param Selection Selected events
	 */
	UE_API FReply OnBarRightClicked( const FGeometry& BarGeometry, const FPointerEvent& MouseEvent, TSharedPtr<FVisualizerEvent> Selection );

	/**
	 * Recursively clears selection on all bar graphs
	 *
	 * @param GraphEvents Bar Graph events
	 * @param Selection Currently selected event
	 */
	UE_API void ClearBarSelection( TSharedPtr< FVisualizerEvent > GraphEvents, TSharedPtr<FVisualizerEvent> Selection );

	/** Creates filtered data */
	UE_API void CreateDataView();

	/** Creates flattened data view */
	UE_API void CreateFlattenedData( TSharedPtr< FVisualizerEvent > InData, TArray< TSharedPtr< FVisualizerEvent > >& FlattenedData );

	/** 
	 * Sets the current view mode 
	 *
	 * @param InMode New view mode
	 */
	UE_API void SetViewMode( EVisualizerViewMode::Type InMode );

	/** 
	 * Given a view mode checks if it's the currently selected one 
	 *
	 * @param InMode View mode to check
	 */
	bool CheckViewMode( EVisualizerViewMode::Type InMode ) const
	{
		return (ViewMode == InMode);
	}

	/** Handles clicking on 'Back to parent' button */
	UE_API FReply OnToParentClicked();

	/** Handles clickong on 'Home' button */
	UE_API FReply OnHomeClicked();

	/** Called when bar graph geometry (size) changes */
	UE_API void OnBarGeometryChanged( FGeometry Geometry );

	/** Gets the currently displayed hierarchy node name */
	UE_API FText GetSelectedCategoryName() const;

	/** Checks if home button should be visible */
	UE_API EVisibility GetHomeButtonVisibility() const;	

	/** Checks if 'to parent' button should be visible */
	UE_API EVisibility GetToParentButtonVisibility() const;

	/** Called when the user clicked bar graph's expand button */
	UE_API FReply ExpandBar( TSharedPtr<FVisualizerEvent> BarGraphEvents );

	/** Checks if the selected event has children with children */
	UE_API bool IsExpandable( TSharedPtr< FVisualizerEvent > InEvent );

	/** Adjusts timeline to match the selected event's start and duration */
	UE_API void AdjustTimeline( TSharedPtr< FVisualizerEvent > InEvent );

	UE_API TSharedPtr< FVisualizerEvent > FindSelectedEventsParent( TArray< TSharedPtr< FVisualizerEvent > >& BarGraphs, TSharedPtr< FVisualizerEvent > Selection );

	/** A pointer to the ListView of threads graph bars */
	TSharedPtr< SListView< TSharedPtr< FVisualizerEvent > > > BarGraphsList;

	/** Currently selected bar graph */
	TSharedPtr< FVisualizerEvent > SelectedBarGraph;

	/** Original profiler data */
	TSharedPtr< FVisualizerEvent > ProfileData;

	/** Profiler data view (filtered data) */
	TArray< TSharedPtr< FVisualizerEvent > > ProfileDataView;

	/** List of all SGraphBar widgets in the tree */
	TArray< TSharedPtr< SGraphBar > > Graphs;

	/** A pointer to the Zoom Label widget */
	TSharedPtr< STextBlock > ZoomLabel;

	/** A pointer to the horizontal scrollbar widget */
	TSharedPtr< SScrollBar > ScrollBar;

	/** A pointer to the horizontal scrollbar widget */
	TSharedPtr< STimeline > Timeline;

	/** Zoom slider value */
	float ZoomSliderValue;

	/** Scrollbar offset */
	float ScrollbarOffset;

	/** Should the OnBarGraphSelectionChangedDelegate be suppressed to avoid event loops */
	bool bSuppressBarGraphSelectionChangedDelegate;

	/** Delegate used when bar graph selection changes */
	FOnBarGraphSelectionChanged OnBarGraphSelectionChangedDelegate;

	/** Delegate used when bar graph selection changes */
	FOnBarGraphExpansionChanged OnBarGraphExpansionChangedDelegate;

	/** Delegate used when single event on the bar graph is selected */
	FOnBarEventSelectionChanged OnBarEventSelectionChangedDelegate;

	/** Delegate used when the user right-clicks on a bar graph */
	FOnBarGraphContextMenu OnBarGraphContextMenuDelegate;

	/** Bar visualizer view mode */
	EVisualizerViewMode::Type ViewMode;

};

#undef UE_API
