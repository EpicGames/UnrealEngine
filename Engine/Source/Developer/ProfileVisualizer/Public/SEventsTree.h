// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "VisualizerEvents.h"

#define UE_API PROFILEVISUALIZER_API

/**
 * Item used in task graph events list
 */
class SEventItem : public SMultiColumnTableRow< TSharedPtr< FVisualizerEvent > >
{
public:
	
	SLATE_BEGIN_ARGS( SEventItem )
		: _EventName()
		, _EventDuration(0.0)
		{}


		SLATE_ARGUMENT( FString, EventName )

		SLATE_ATTRIBUTE( double, EventDuration )		

	SLATE_END_ARGS()
	
	/**
	 * Generates a widget for task graph events list column
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API virtual TSharedRef< SWidget > GenerateWidgetForColumn( const FName& ColumnName ) override;


	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView );

private:

	FText GetDurationText() const
	{
		static const FNumberFormattingOptions DurationFormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(2)
			.SetMaximumFractionalDigits(2);
		return FText::AsNumber(EventDuration.Get(), &DurationFormatOptions);
	}

	FString EventName;
	TAttribute< double > EventDuration;
};


/**
 * Profiler events tree. Contains a tree of profiler event for the selected graph bar.
 */
class SEventsTree : public SCompoundWidget
{
public:

	/** Delegate used when the selection changes */
	DECLARE_DELEGATE_OneParam( FOnEventSelectionChanged, TSharedPtr< FVisualizerEvent > );

	SLATE_BEGIN_ARGS( SEventsTree )
		: _ProfileData()
		, _OnEventSelectionChanged()
		{}

		/** Profiler data */
		SLATE_ATTRIBUTE( TSharedPtr< FVisualizerEvent >, ProfileData )

		/** Event for handling selection changes */
		SLATE_EVENT( FOnEventSelectionChanged, OnEventSelectionChanged )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	/**
	 * Function called when the currently selected bar graph changes
	 *
	 * @param Selection Currently selected bar graph
	 */
	UE_API void HandleBarGraphSelectionChanged( TSharedPtr< FVisualizerEvent > Selection );

	/**
	 * Function called when the currently expanded bar graph changes
	 *
	 * @param Selection Currently expanded bar graph
	 */
	UE_API void HandleBarGraphExpansionChanged( TSharedPtr< FVisualizerEvent > Selection );

	/**
	 * Function called when the user selects an event bar in the graph
	 *
	 * @param Selection Currently selected event
	 */
	UE_API void HandleBarEventSelectionChanged( int32 Thread, TSharedPtr< FVisualizerEvent > Selection );

	/** Gets a label for this events tree tab */
	UE_API FString GetTabTitle() const;

	/** Name of 'Name' column */
	static UE_API FName NAME_NameColumn;

	/** Name of 'Duration' column */
	static UE_API FName NAME_DurationColumn;

protected:

	/**
	 * Function called when the currently selected event in the list of thread events changes
	 *
	 * @param Selection Currently selected event
	 * @param SelectInfo Provides context on how the selection changed
	 */
	UE_API void OnEventSelectionChanged( TSharedPtr< FVisualizerEvent > Selection, ESelectInfo::Type SelectInfo );

	/** 
	 * Generates SEventItem widgets for the events tree
	 *
	 * @param InItem Event to generate SEventItem for
	 * @param OwnerTable Owner Table
	 */
	UE_API TSharedRef< ITableRow > OnGenerateWidgetForEventsList( TSharedPtr< FVisualizerEvent > InItem, const TSharedRef< STableViewBase >& OwnerTable );

	/** 
	 * Given a profiler event, generates children for it
	 *
	 * @param InItem Event to generate children for
	 * @param OutChildren Generated children
	 */
	UE_API void OnGetChildrenForEventsList( TSharedPtr<FVisualizerEvent> InItem, TArray<TSharedPtr<FVisualizerEvent> >& OutChildren );

	/** 
	 * Handles column sorting mode change
	 *
	 * @param InSortMode New sorting mode
	 */
	UE_API void OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode );

	/**
	 * @return Returns the current sort mode of the specified column
	 */
	UE_API EColumnSortMode::Type GetColumnSortMode( const FName ColumnId );

	/** Sorts the selected events tree */
	UE_API void SortEventsList( void );

	/** Recursively sorts the selected events tree */
	UE_API void SortEventsList( TArray< TSharedPtr< FVisualizerEvent > >& Events );

	/** Restores event selection after changes to the tree */
	UE_API bool RestoreEventSelection( TArray< TSharedPtr< FVisualizerEvent > >& Events );

	/** 
	 * Sets 'Duration' column time units
	 *
	 * @param InUnits New units
	 */
	UE_API void SetDurationUnits( EVisualizerTimeUnits::Type InUnits );

	/** Recursively counts all events in the tree */
	UE_API int32 CountEvents( TArray< TSharedPtr< FVisualizerEvent > >& Events );

	/** Given the selected events from the bar graph creates a copy of the selection applying the current view and sorting mode */
	UE_API void CreateSelectedEventsView();

	/** Helper function for creating a copy of the selected events in a hierarchy */
	UE_API TSharedPtr< FVisualizerEvent > CreateSelectedEventsViewRecursively( TSharedPtr< FVisualizerEvent > SourceEvent );

	/** Helper function for creating a copy of the selected events and flattening the hierarchy */
	UE_API void CreateSelectedEventsViewRecursivelyAndFlatten( TSharedPtr< FVisualizerEvent > SourceEvent );

	/** Helper function for creating a copy of the selected events combining leaves with the same name */
	UE_API void CreateSelectedEventsViewRecursivelyCoalesced( TArray< TSharedPtr< FVisualizerEvent > >& SourceEvents, TArray< TSharedPtr< FVisualizerEvent > >& CopiedEvents, TSharedPtr< FVisualizerEvent > InParent );
	
	/** Helper function for creating a copy of the selected events combining leaves with the same name and flattening the hierarchy */
	UE_API void CreateSelectedEventsViewRecursivelyFlatCoalesced( TArray< TSharedPtr< FVisualizerEvent > >& SourceEvents );

	/** Gets the currently selected time units */
	bool CheckDurationUnits( EVisualizerTimeUnits::Type InUnits ) const
	{
		return (InUnits == DurationUnits);
	}

	/** 
	 * Sets the current view mode 
	 *
	 * @param InMode New view mode
	 */
	UE_API void SetViewMode( EVisualizerViewMode::Type InMode );

	/** 
	 * Given a view mode checks if it's the currently selected one 
	 *
	 * @param InMode View mode to check
	 */
	bool CheckViewMode( EVisualizerViewMode::Type InMode ) const
	{
		return (ViewMode == InMode);
	}

	/** 
	 * Converts ms to currently selected time units.
	 *
	 * @param InDurationMs duration time in ms.
	 */
	UE_API double GetEventDuration( double InDurationMs ) const;

	/**
		* Called by the editable text control when the filter text is changed by the user
		*
		* @param	InFilterText	The new text
		*/
	UE_API void OnFilterTextChanged( const FText& InFilterText );

	/** Called by the editable text control when a user presses enter or commits their text change */
	UE_API void OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo );

	/** Checks if event name passes current filter */
	bool FilterEvent( TSharedPtr< FVisualizerEvent > InEvent ) const
	{
		// Filter only leaves, we want to keep the hierarchy
		return InEvent->Children.Num() > 0 || FilterText.Len() == 0 || InEvent->EventName.StartsWith( FilterText );
	}

	/** Gets the currently selected time units text */
	UE_API FText GetDurationColumnTitle() const;

	/** A pointer to the ListView of profiler events */
	TSharedPtr< STreeView< TSharedPtr< FVisualizerEvent > > > EventsListView;

	/** Original profiler data */
	TSharedPtr< FVisualizerEvent > ProfileData;

	/** Currently selected events for this tree */
	TSharedPtr< FVisualizerEvent > SelectedThread;

	/** List of events for the currently selected thread */
	TArray< TSharedPtr< FVisualizerEvent > > SelectedEvents;

	/** List of events for the currently selected thread */
	TArray< TSharedPtr< FVisualizerEvent > > SelectedEventsView;

	/** Maps the events generated using currently selected view and sorting modes to the source events tree */
	TMap< TSharedPtr< FVisualizerEvent >, TSharedPtr< FVisualizerEvent > > ViewToEventsMap;

	/** Delegate used to notify when event selection changes */
	FOnEventSelectionChanged OnEventSelectionChangedDelegate;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	/** Currently selected time units */
	EVisualizerTimeUnits::Type DurationUnits;

	/** Currently selected view mode */
	EVisualizerViewMode::Type ViewMode;

	/** Events filter text */
	FString FilterText;

	/** Suppresses SelectionChanged delegate to avoid event loops between graph visualizer and events tree */
	bool bSuppressSelectionChangedEvent;
};

#undef UE_API
