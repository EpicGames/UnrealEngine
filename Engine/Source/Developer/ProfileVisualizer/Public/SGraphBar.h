// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"
#include "VisualizerEvents.h"

#define UE_API PROFILEVISUALIZER_API

class FPaintArgs;
class FSlateWindowElementList;
struct FSlateBrush;

/** A graph bar widget.*/
class SGraphBar : public SLeafWidget
{
public:

	/** Delegate used allotted gemetry changes */
	DECLARE_DELEGATE_OneParam( FOnGeometryChanged, FGeometry );

	typedef TSlateDelegates< TSharedPtr< FVisualizerEvent > >::FOnSelectionChanged FOnSelectionChanged;

public:
	SLATE_BEGIN_ARGS( SGraphBar )
		: _OnSelectionChanged()
		{}

		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )

		SLATE_EVENT( FOnGeometryChanged, OnGeometryChanged )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	// End of SWidget interface

	/**
	 * Adds profiler events to draw as bars
	 *
	 * @param InEvent	Events to draw
	 */
	UE_API void SetEvents( const FVisualizerEventsArray& InEvents, double InStartTime, double InTotalTime );

	/**
	 * Sets the graph's zoom level
	 *
	 * @param NewValue	Zoom value, minimum is 1.0
	 */
	void SetZoom( float InZoom )
	{
		Zoom = FMath::Max(InZoom, 1.0f);
	}

	/**
	 * Sets the graph's offset by which all graph bars should be moved
	 *
	 * @param NewValue	Offset value
	 */
	void SetOffset( float InOffset )
	{
		Offset = InOffset;
	} 

	/**
	 * Gets the graph's offset value
	 *
	 * @return Offset value
	 */
	float GetOffset() const
	{
		return Offset;
	}

	/**
	 * Gets all events this graph will draw
	 *
	 * @return List of all events to draw
	 */
	FVisualizerEventsArray& GetEvents()
	{
		return Events;
	}

private:

	FORCEINLINE bool CalculateEventGeometry( FVisualizerEvent* InEvent, const FGeometry& InGeometry, float& OutStartX, float& OutEndX ) const
	{
		const double SubPixelMinSize = 3.0;
		const double EventStart = (InEvent->Start - StartTime) / TotalTime;
		const double EventDuration = InEvent->Duration / TotalTime;
		const double ClampedStart = FMath::Clamp( Offset + EventStart * Zoom, 0.0, 1.0  );		
		const double ClampedEnd = FMath::Clamp( Offset + (EventStart + EventDuration) * Zoom, 0.0, 1.0  );
		const double ClampedSize = ClampedEnd - ClampedStart;
		OutStartX = (float)( InGeometry.GetLocalSize().X * ClampedStart );
		OutEndX = OutStartX + (float)FMath::Max( InGeometry.GetLocalSize().X * ClampedSize, ClampedEnd > 0.0f ? SubPixelMinSize : 0.0 );
		return ClampedEnd > 0.0 && ClampedStart < 1.0;
	}

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnSelectionChanged;

	/** Background image to use for the graph bar */
	const FSlateBrush* BackgroundImage;

	/** Foreground image to use for the graph bar */
	const FSlateBrush* FillImage;

	/** Image to be used when drawing selected event */
	const FSlateBrush* SelectedImage;

	/** List of all events to draw */
	FVisualizerEventsArray Events;

	/** Start time (0.0 - 1.0) */
	double StartTime;
	
	/** End time (0.0 - 1.0) */
	double TotalTime;

	/** Current zoom of the graph */
	float Zoom;

	/** Current offset of the graph */
	float Offset;

	/** Last hovered event index */
	int32 LastHoveredEvent;

	/** Last allotted geometry */
	FGeometry LastGeometry;

	/** Delegate called when the geometry changes */
	FOnGeometryChanged OnGeometryChanged;

	/** Color palette for bars coloring */
	static UE_API FColor	ColorPalette[];
};

#undef UE_API
