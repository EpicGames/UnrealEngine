// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API PROFILEVISUALIZER_API

class FPaintArgs;
class FSlateWindowElementList;
struct FSlateBrush;

/** A timeline widget.*/
class STimeline : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS( STimeline )
		: _MinValue( 0.0f )
		, _MaxValue( 1.0f )
		, _FixedLabelSpacing(1.0)
		{}

		/** Minimum value on the timeline */
		SLATE_ARGUMENT( float, MinValue )

		/** Maximum value on the timeline */
		SLATE_ARGUMENT( float, MaxValue )

		/** fixed pixel spacing between centers of labels */
		SLATE_ARGUMENT( float, FixedLabelSpacing )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	UE_API virtual int32 OnPaint( const FPaintArgs& Args,  const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	// End of SWidget interface

	/**
	 * Sets the graph's zoom level
	 *
	 * @param NewValue	Zoom value, minimum is 1.0
	 */
	void SetZoom( float InZoom )
	{
		Zoom = FMath::Max(InZoom, 1.0f);
	}

	float GetZoom()
	{
		return Zoom;
	}
	/**
	 * Sets the graph's offset by which all graph bars should be moved
	 *
	 * @param NewValue	Offset value
	 */
	void SetOffset( float InOffset )
	{
		Offset = InOffset;
	} 

	/**
	 * Gets the graph's offset value
	 *
	 * @return Offset value
	 */
	float GetOffset() const
	{
		return Offset;
	}

	/**
	 * Sets the graph's min and max values.
	 *
	 * @param InMin	New min value.
	 * @param InMin	New max value.
	 */
	void SetMinMaxValues(float InMin, float InMax)
	{
		MinValue = InMin;
		MaxValue = InMax;
	}

	void GetMinMaxValues(float &InMin, float &InMax)
	{
		InMin = MinValue;
		InMax = MaxValue;
	}

	void SetDrawingGeometry(const FGeometry& Geometry)
	{
		DrawingGeometry = Geometry;
	}

	FGeometry GetDrawingGeometry() const
	{
		return DrawingGeometry;
	}

private:

	/** Background image to use for the graph bar */
	const FSlateBrush* BackgroundImage;

	/** Minimum value on the timeline */
	float MinValue;

	/** Maximum value on the timeline */
	float MaxValue;

	/** fixed pixel spacing between centers of labels */
	float FixedLabelSpacing;

	/** Current zoom of the graph */
	float Zoom;

	/** Current offset of the graph */
	float Offset;

	float DrawingOffsetX;

	FGeometry DrawingGeometry;
};

#undef UE_API
