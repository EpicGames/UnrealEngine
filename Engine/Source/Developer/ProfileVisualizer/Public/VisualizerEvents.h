// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Templates/SharedPointer.h"

class FArchive;

namespace EVisualizerTimeUnits
{
	typedef uint8 Type;

		/** Microseconds */
	const Type Microseconds = 0;

		/** Milliseconds */
	const Type Milliseconds = 1;

		/** Seconds */
	const Type Seconds = 2;
};

namespace EVisualizerViewMode
{
	typedef uint8 Type;

	/** Hierarchical */
	const Type Hierarchical = 0;

	/** Flat */
	const Type Flat = 1;

	/** Coalesced */
	const Type Coalesced = 2;

	/** FlatCoalesced */
	const Type FlatCoalesced = 3;
};

/** A graph event represented by SGraphBar as a single bar.*/
struct FVisualizerEvent
{	
	/** Normalized start time (0.0-1.0) of the event relative to the first event in the profile */
	double Start;

	/** Normalized duration time (0.0-1.0) of the event. */
	double Duration;

	/** Duration of the event in milliseconds */
	double DurationMs;

	/** Category this event belongs to (thread/file etc.) */
	int32 Category;

	/** Name of the event. */
	FString EventName;

	/** Determines if this event is selected or not. */
	bool IsSelected;

	/** Bar color */
	uint32 ColorIndex;

	/** Parent event */
	TSharedPtr< FVisualizerEvent > ParentEvent;

	/**  Child events */
	TArray< TSharedPtr< FVisualizerEvent > > Children;

	FVisualizerEvent(const double InStart, const double InDuration, const double InDurationMs, const int32 InCategory, const FString& InEventName)
		: Start( InStart )
		, Duration( InDuration )
		, DurationMs( InDurationMs )
		, Category( InCategory )
		, EventName( InEventName )
		, IsSelected( false )
		, ColorIndex( 0 )
	{
		ColorIndex = GetTypeHash( InEventName );
	}

	static TSharedPtr< FVisualizerEvent > LoadVisualizerEvent(FArchive *Ar);
	static void SaveVisualizerEventRecursively(FArchive *Ar, TSharedPtr< FVisualizerEvent > VisualizerEvent);

private:
	static TSharedPtr< FVisualizerEvent > LoadVisualizerEventRecursively(FArchive *Ar, TSharedPtr< FVisualizerEvent > InParentEvent);

};

/** Array of graph events.*/
typedef TArray< TSharedPtr< FVisualizerEvent > > FVisualizerEventsArray;
